// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class EOSVoiceChat_WinGDK : EOSVoiceChat
	{
		public EOSVoiceChat_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
