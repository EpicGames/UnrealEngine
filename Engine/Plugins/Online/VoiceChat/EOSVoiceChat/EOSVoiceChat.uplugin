{
	"FileVersion": 3,

	"FriendlyName": "EOS Voice Chat",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "IVoiceChat integration of the EOS Voice service",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "EOSVoiceChat",
			"Type": "ClientOnlyNoCommandlet",
			"PlatformAllowList": [
				"Android",
				"Mac",
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "EOSShared",
			"Enabled": true
		},
		{
			"Name": "VoiceChat",
			"Enabled": true
		}
	]
}
