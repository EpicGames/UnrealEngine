// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class EOSVoiceChat : ModuleRules
	{
		[ConfigFile(ConfigHierarchyType.Engine, "EOSVoiceChat")]
		bool bDisableInMonolithic = false;

		bool DisableInMonolithic
		{
			get
			{
				// Keep supporting old GlobalDef mechanism in case licensees are using it
				if (Target.GlobalDefinitions.Contains("DISABLE_EOSVOICECHAT_ENGINE=1"))
				{
					Logger.LogWarning("DISABLE_EOSVOICECHAT_ENGINE is deprecated, please use Engine.ini [EOSVoiceChat] bDisableInMonolithic=true config mechanism instead");
					return true;
				}

				ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);
				return bDisableInMonolithic;
			}
		}

		public EOSVoiceChat(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			if (DisableInMonolithic)
			{
				bool bIsMonolithic = Target.LinkType == TargetLinkType.Monolithic;
				bool bIsUniqueBuildEnv = Target.BuildEnvironment == TargetBuildEnvironment.Unique;
				if(bIsMonolithic || bIsUniqueBuildEnv)
				{
					PublicDefinitions.Add("WITH_EOSVOICECHAT=0");
				}
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Json",
					"VoiceChat"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Projects",
					"EOSShared",
					"EOSSDK"
				}
			);

			if(Target.Platform == UnrealTargetPlatform.IOS)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"ApplicationCore"
					}
				);
			}
		}
	}
}
