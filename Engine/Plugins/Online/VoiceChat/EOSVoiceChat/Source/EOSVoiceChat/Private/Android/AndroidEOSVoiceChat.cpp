// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidEOSVoiceChat.h"

#if WITH_EOSVOICECHAT

#include "AndroidEOSVoiceChatUser.h"

FAndroidEOSVoiceChat::FAndroidEOSVoiceChat(IEOSSDKManager& InSDKManager)
	: FEOSVoiceChat(InSDKManager)
{
}

IVoiceChatUser* FAndroidEOSVoiceChat::CreateUser()
{
	const FEOSVoiceChatUserRef& User = VoiceChatUsers.Emplace_GetRef(MakeShared<FAndroidEOSVoiceChatUser, ESPMode::ThreadSafe>(*this));
	return &User.Get();
}

#endif // WITH_EOSVOICECHAT