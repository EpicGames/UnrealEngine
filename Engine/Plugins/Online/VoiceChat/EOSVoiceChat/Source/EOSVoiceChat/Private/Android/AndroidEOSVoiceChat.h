// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

class FAndroidEOSVoiceChat : public FEOSVoiceChat
{
public:
	FAndroidEOSVoiceChat(IEOSSDKManager& InSDKManager);
	virtual ~FAndroidEOSVoiceChat() = default;

	// ~Begin IVoiceChat Interface 
	virtual IVoiceChatUser* CreateUser() override;
	// ~End IVoiceChat Interface
};

using FPlatformEOSVoiceChat = FAndroidEOSVoiceChat;

#endif // WITH_EOSVOICECHAT