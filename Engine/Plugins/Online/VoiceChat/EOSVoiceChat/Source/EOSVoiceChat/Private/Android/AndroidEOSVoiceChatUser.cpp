// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidEOSVoiceChatUser.h"

#if WITH_EOSVOICECHAT

void FAndroidEOSVoiceChatUser::SetSetting(const FString& Name, const FString& Value)
{
	if (Name == TEXT("BluetoothMicrophone"))
	{
		bEnableBluetoothMicrophone = FCString::ToBool(*Value);
		// TODO
	}
	else
	{
		FEOSVoiceChatUser::SetSetting(Name, Value);
	}
}

FString FAndroidEOSVoiceChatUser::GetSetting(const FString& Name)
{
	if (Name == TEXT("BluetoothMicrophone"))
	{
		return LexToString(bEnableBluetoothMicrophone);
	}
	else
	{
		return FEOSVoiceChatUser::GetSetting(Name);
	}
}

#endif // WITH_EOSVOICECHAT