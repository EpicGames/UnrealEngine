// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChatUser.h"

#if WITH_EOSVOICECHAT

class FAndroidEOSVoiceChatUser : public FEOSVoiceChatUser
{
public:
	FAndroidEOSVoiceChatUser(FEOSVoiceChat& InEOSVoiceChat) : FEOSVoiceChatUser(InEOSVoiceChat) {}
	virtual ~FAndroidEOSVoiceChatUser() = default;

	// ~Begin IVoiceChatUser Interface
	virtual void SetSetting(const FString& Name, const FString& Value) override;
	virtual FString GetSetting(const FString& Name) override;
	// ~End IVoiceChatUser Interface

private:
	bool bEnableBluetoothMicrophone = false;
};

#endif // WITH_EOSVOICECHAT