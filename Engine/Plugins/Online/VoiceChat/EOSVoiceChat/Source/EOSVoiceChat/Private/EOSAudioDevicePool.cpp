// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSAudioDevicePool.h"

#if WITH_EOSVOICECHAT

#include "EOSVoiceChatLog.h"

#include "eos_rtc.h"
#include "eos_rtc_audio.h"

namespace EOSAudioDevicePoolPrivate
{
	// State while handling a refresh request
	// firing input `Delegate` during destroying
	struct FRefreshAudioDevicesState
	{
		UE_NONCOPYABLE(FRefreshAudioDevicesState)

		using FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate = FEOSAudioDevicePool::FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate;

		explicit FRefreshAudioDevicesState(FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate InDelegate)
		: Delegate(MoveTemp(InDelegate))
		{
		}

		~FRefreshAudioDevicesState()
		{
			Delegate.ExecuteIfBound(MoveTemp(Result));
		}

		FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
		const FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate Delegate;
	};
}

FEOSAudioDevicePool::FEOSAudioDevicePool(EOS_HRTC& InEosRtcInterface)
	: EosRtcInterface{ InEosRtcInterface }
{
}

void FEOSAudioDevicePool::RefreshAudioDevices(const FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate& Delegate)
{
	// Delegate will be shared between several async calls and be invoked on the last one (in destructor as the last reference)
	// with a success result if no one has been failed and failure otherwise
	TSharedRef<EOSAudioDevicePoolPrivate::FRefreshAudioDevicesState> RefreshAudioDevicesState = MakeShared<EOSAudioDevicePoolPrivate::FRefreshAudioDevicesState>(Delegate);

	RefreshAudioInputDevices(FOnAudioDevicePoolRefreshAudioInputDevicesCompleteDelegate::CreateLambda([RefreshAudioDevicesState](const FVoiceChatResult& InResult) -> void
	{
		if (!InResult.IsSuccess())
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "RefreshAudioInputDevicesCompletionDelegate failed, Result=[%ls]", *LexToString(InResult));

			RefreshAudioDevicesState->Result = InResult;
		}
	}));
	RefreshAudioOutputDevices(FOnAudioDevicePoolRefreshAudioOutputDevicesCompleteDelegate::CreateLambda([RefreshAudioDevicesState](const FVoiceChatResult& InResult) -> void
	{
		if (!InResult.IsSuccess())
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "RefreshAudioOutputDevicesCompletionDelegate failed, Result=[%ls]", *LexToString(InResult));

			RefreshAudioDevicesState->Result = InResult;
		}
	}));
}

void FEOSAudioDevicePool::RefreshAudioInputDevices(const FOnAudioDevicePoolRefreshAudioInputDevicesCompleteDelegate& Delegate)
{
	CachedInputDeviceInfos = GetRtcInputDeviceInfos(DefaultInputDeviceInfoIdx);

	UE_LOGF(LogEOSVoiceChat, Verbose, "RefreshAudioInputDevices InputDeviceInfos=[%ls] DefaultInputDeviceInfoIdx=[%d]", *FString::JoinBy(CachedInputDeviceInfos, TEXT(", "), &FVoiceChatDeviceInfo::ToDebugString), DefaultInputDeviceInfoIdx);

	Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
}

void FEOSAudioDevicePool::RefreshAudioOutputDevices(const FOnAudioDevicePoolRefreshAudioOutputDevicesCompleteDelegate& Delegate)
{
	CachedOutputDeviceInfos = GetRtcOutputDeviceInfos(DefaultOutputDeviceInfoIdx);

	UE_LOGF(LogEOSVoiceChat, Verbose, "RefreshAudioOutputDevices OutputDeviceInfos=[%ls] DefaultOutputDeviceInfoIdx=[%d]", *FString::JoinBy(CachedOutputDeviceInfos, TEXT(", "), &FVoiceChatDeviceInfo::ToDebugString), DefaultOutputDeviceInfoIdx);

	Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
}

TArray<FVoiceChatDeviceInfo> FEOSAudioDevicePool::GetRtcInputDeviceInfos(int32& OutDefaultDeviceIdx) const
{
	TArray<FVoiceChatDeviceInfo> InputDeviceInfos;
	EOS_HRTCAudio RTCAudioHandle = EOS_RTC_GetAudioInterface(EosRtcInterface);

	EOS_RTCAudio_GetAudioInputDevicesCountOptions CountOptions = {};
	CountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_GETAUDIOINPUTDEVICESCOUNT_API_LATEST, 1);

	uint32_t Count = EOS_RTCAudio_GetAudioInputDevicesCount(RTCAudioHandle, &CountOptions);

	for (uint32_t Index = 0; Index < Count; Index++)
	{
		EOS_RTCAudio_GetAudioInputDeviceByIndexOptions GetByIndexOptions = {};
		GetByIndexOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_GETAUDIOINPUTDEVICEBYINDEX_API_LATEST, 1);
		GetByIndexOptions.DeviceInfoIndex = Index;
		if (const EOS_RTCAudio_AudioInputDeviceInfo* DeviceInfo = EOS_RTCAudio_GetAudioInputDeviceByIndex(RTCAudioHandle, &GetByIndexOptions))
		{
			FString DeviceName = UTF8_TO_TCHAR(DeviceInfo->DeviceName);
			if (DeviceName != TEXT("Default Device"))
			{
				FVoiceChatDeviceInfo& InputDeviceInfo = InputDeviceInfos.Emplace_GetRef();
				InputDeviceInfo.DisplayName = MoveTemp(DeviceName);
				InputDeviceInfo.Id = UTF8_TO_TCHAR(DeviceInfo->DeviceId);
				if (DeviceInfo->bDefaultDevice)
				{
					OutDefaultDeviceIdx = InputDeviceInfos.Num() - 1;
				}
			}
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "EOS_RTCAudio_GetAudioInputDeviceByIndex failed: DevicesInfo=nullptr");
		}
	}

	if (Count == 0)
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "EOS_RTCAudio_GetAudioInputDevicesCount failed: DevicesCount=0");
	}

	return InputDeviceInfos;
}

TArray<FVoiceChatDeviceInfo> FEOSAudioDevicePool::GetRtcOutputDeviceInfos(int32& OutDefaultDeviceIdx) const
{
	OutDefaultDeviceIdx = -1;
	TArray<FVoiceChatDeviceInfo> OutputDeviceInfos;
	EOS_HRTCAudio RTCAudioHandle = EOS_RTC_GetAudioInterface(EosRtcInterface);

	EOS_RTCAudio_GetAudioOutputDevicesCountOptions CountOptions = {};
	CountOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_GETAUDIOOUTPUTDEVICESCOUNT_API_LATEST, 1);

	uint32_t Count = EOS_RTCAudio_GetAudioOutputDevicesCount(RTCAudioHandle, &CountOptions);

	for (uint32_t Index = 0; Index < Count; Index++)
	{
		EOS_RTCAudio_GetAudioOutputDeviceByIndexOptions GetByIndexOptions = {};
		GetByIndexOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_GETAUDIOOUTPUTDEVICEBYINDEX_API_LATEST, 1);
		GetByIndexOptions.DeviceInfoIndex = Index;
		if (const EOS_RTCAudio_AudioOutputDeviceInfo* DeviceInfo = EOS_RTCAudio_GetAudioOutputDeviceByIndex(RTCAudioHandle, &GetByIndexOptions))
		{
			FString DeviceName = UTF8_TO_TCHAR(DeviceInfo->DeviceName);
			if (DeviceName != TEXT("Default Device"))
			{
				FVoiceChatDeviceInfo& InputDeviceInfo = OutputDeviceInfos.Emplace_GetRef();
				InputDeviceInfo.DisplayName = MoveTemp(DeviceName);
				InputDeviceInfo.Id = UTF8_TO_TCHAR(DeviceInfo->DeviceId);
				if (DeviceInfo->bDefaultDevice)
				{
					OutDefaultDeviceIdx = OutputDeviceInfos.Num() - 1;
				}
			}
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "EOS_RTCAudio_GetAudioOutputDeviceByIndex failed: DevicesInfo=nullptr");
		}
	}

	if (Count == 0)
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "EOS_RTCAudio_GetAudioOutputDevicesCount failed: DevicesCount=0");
	}

	return OutputDeviceInfos;
}

const TArray<FVoiceChatDeviceInfo>& FEOSAudioDevicePool::GetCachedInputDeviceInfos() const
{
	return CachedInputDeviceInfos;
}

const TArray<FVoiceChatDeviceInfo>& FEOSAudioDevicePool::GetCachedOutputDeviceInfos() const
{
	return CachedOutputDeviceInfos;
}

int32 FEOSAudioDevicePool::GetDefaultInputDeviceInfoIdx() const
{
	return DefaultInputDeviceInfoIdx;
}

int32 FEOSAudioDevicePool::GetDefaultOutputDeviceInfoIdx() const
{
	return DefaultOutputDeviceInfoIdx;
}

#endif // WITH_EOSVOICECHAT
