// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

class FEOSAudioDevicePool final : public TSharedFromThis<FEOSAudioDevicePool>
{
public:
	explicit FEOSAudioDevicePool(EOS_HRTC& InEosRtcInterface);

	// forward refreshing both input and output devices
	DECLARE_DELEGATE_OneParam(FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate, const FVoiceChatResult& /* Result */);
	void RefreshAudioDevices(const FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate& Delegate);

	DECLARE_DELEGATE_OneParam(FOnAudioDevicePoolRefreshAudioInputDevicesCompleteDelegate, const FVoiceChatResult& /* Result */);
	void RefreshAudioInputDevices(const FOnAudioDevicePoolRefreshAudioInputDevicesCompleteDelegate& Delegate);

	DECLARE_DELEGATE_OneParam(FOnAudioDevicePoolRefreshAudioOutputDevicesCompleteDelegate, const FVoiceChatResult& /* Result */);
	void RefreshAudioOutputDevices(const FOnAudioDevicePoolRefreshAudioOutputDevicesCompleteDelegate& Delegate);

	const TArray<FVoiceChatDeviceInfo>& GetCachedInputDeviceInfos() const;
	const TArray<FVoiceChatDeviceInfo>& GetCachedOutputDeviceInfos() const;

	int32 GetDefaultInputDeviceInfoIdx() const;
	int32 GetDefaultOutputDeviceInfoIdx() const;
private:
	TArray<FVoiceChatDeviceInfo> GetRtcInputDeviceInfos(int32& OutDefaultDeviceIdx) const;
	TArray<FVoiceChatDeviceInfo> GetRtcOutputDeviceInfos(int32& OutDefaultDeviceIdx) const;

	EOS_HRTC& EosRtcInterface;

	int32 DefaultInputDeviceInfoIdx = -1;
	int32 DefaultOutputDeviceInfoIdx = -1;
	TArray<FVoiceChatDeviceInfo> CachedInputDeviceInfos;
	TArray<FVoiceChatDeviceInfo> CachedOutputDeviceInfos;
};

#endif // WITH_EOSVOICECHAT
