// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVoiceChat.h" 

#if WITH_EOSVOICECHAT

#include "Async/Async.h"
#include "Containers/Ticker.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"

#include "EOSShared.h"
#include "EOSVoiceChatLog.h"
#include "EOSVoiceChatModule.h"
#include "EOSVoiceChatUser.h"
#include "EOSAudioDevicePool.h"
#include "IEOSSDKManager.h"
#include "VoiceChatErrors.h"
#include "VoiceChatResult.h"

#include "eos_sdk.h"
#include "eos_rtc.h"
#include "eos_rtc_audio.h"

#define CONFIG_SECTION_NAME TEXT("EOSVoiceChat")
#define CONFIG_FILE GEngineIni
#define CHECKPIN() FEOSVoiceChatPtr StrongThis = WeakThis.Pin(); if(!StrongThis) return

DEFINE_LOG_CATEGORY(LogEOSVoiceChat);

FEOSVoiceChatDelegates::FOnAudioInputDeviceStatusChanged FEOSVoiceChatDelegates::OnAudioInputDeviceStatusChanged;
FEOSVoiceChatDelegates::FOnVoiceChatPlayerAddedMetadataDelegate FEOSVoiceChatDelegates::OnVoiceChatPlayerAddedMetadata;
FEOSVoiceChatDelegates::FOnAudioStatusChanged FEOSVoiceChatDelegates::OnAudioStatusChanged;

int64 FEOSVoiceChat::StaticInstanceIdCount = 0;

#define EOS_VOICE_TODO 0

// Added for tests
#if !UE_BUILD_SHIPPING
namespace
{
enum class EEOSVoiceInitializationPath : uint8
{
	None,
	PlatformHandleInjected,  // SetPlatformHandle
	NamedConfig,             // SDKManager.CreatePlatform(ConfigName)
	LegacyInlineConfig       // EOSPlatformCreate with inline credentials
};

const TCHAR* LexToString(EEOSVoiceInitializationPath Path)
{
	switch (Path)
	{
		default: case EEOSVoiceInitializationPath::None:			return TEXT("None");
		case EEOSVoiceInitializationPath::PlatformHandleInjected:	return TEXT("PlatformHandleInjected");
		case EEOSVoiceInitializationPath::NamedConfig:				return TEXT("NamedConfig");
		case EEOSVoiceInitializationPath::LegacyInlineConfig:		return TEXT("LegacyInlineConfig");
	}
}
}
#endif // !UE_BUILD_SHIPPING

FEOSVoiceChat::FEOSVoiceChat(IEOSSDKManager& InSDKManager, const IEOSPlatformHandlePtr& InPlatformHandle)
	: SDKManager(InSDKManager)
{
	SetPlatformHandle(InPlatformHandle);
#if !UE_BUILD_SHIPPING
	InitializationPath = static_cast<uint8>(EEOSVoiceInitializationPath::None);
#endif // !UE_BUILD_SHIPPING
}

FEOSVoiceChat::FEOSVoiceChat(IEOSSDKManager& InSDKManager)
	: SDKManager(InSDKManager)
{
}

FEOSVoiceChat::~FEOSVoiceChat()
{
}

#pragma region IVoiceChat
bool FEOSVoiceChat::Initialize()
{
	if (!IsInitialized())
	{
		Initialize(FOnVoiceChatInitializeCompleteDelegate());
	}

	return IsInitialized();
}

bool FEOSVoiceChat::Uninitialize()
{
	bool bIsDone = false;
	Uninitialize(FOnVoiceChatUninitializeCompleteDelegate::CreateLambda([&bIsDone](const FVoiceChatResult& Result)
	{
		bIsDone = true;
	}));

	while (!bIsDone)
	{
		EosPlatformHandle->Tick();
	}
	
	return !IsInitialized();
}

void FEOSVoiceChat::Initialize(const FOnVoiceChatInitializeCompleteDelegate& InitCompleteDelegate)
{
	FVoiceChatResult Result(FVoiceChatResult::CreateSuccess());

	switch (InitSession.State)
	{
	case EInitializationState::Uninitialized:
	{
		bool bEnabled = true;
		GConfig->GetBool(CONFIG_SECTION_NAME, TEXT("bEnabled"), bEnabled, CONFIG_FILE);
		FString PlatformConfigName;
		if (OverridePlatformConfigName.IsSet())
		{
			PlatformConfigName = OverridePlatformConfigName.GetValue();
		}
		else if (!GConfig->GetString(CONFIG_SECTION_NAME, TEXT("PlatformConfigName"), PlatformConfigName, CONFIG_FILE) || PlatformConfigName.IsEmpty())
		{
			PlatformConfigName = SDKManager.GetDefaultPlatformConfigName();
		}

		if (bEnabled)
		{
			InitSession.State = EInitializationState::Initializing;

#if !UE_BUILD_SHIPPING
			if (EosPlatformHandle)
			{
				InitializationPath = static_cast<uint8>(EEOSVoiceInitializationPath::PlatformHandleInjected);
			}
#endif // !UE_BUILD_SHIPPING

			if (!EosPlatformHandle)
			{
				if (SDKManager.IsInitialized())
				{
					if (!PlatformConfigName.IsEmpty())
					{
						// UE-193389: now the named config mechanism in SDKManager loads all credentials from its own config
						EosPlatformHandle = SDKManager.CreatePlatform(PlatformConfigName);

#if !UE_BUILD_SHIPPING
						InitializationPath = static_cast<uint8>(EEOSVoiceInitializationPath::NamedConfig);
#endif // !UE_BUILD_SHIPPING

					}
					else
					{
						// Legacy path: read credentials inline and build platform options
						FString ConfigProductId;
						FString ConfigSandboxId;
						FString ConfigDeploymentId;
						FString ConfigClientId;
						FString ConfigClientSecret;
						FString ConfigEncryptionKey;
						FString ConfigOverrideCountryCode;
						FString ConfigOverrideLocaleCode;
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("ProductId"), ConfigProductId, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("SandboxId"), ConfigSandboxId, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("DeploymentId"), ConfigDeploymentId, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("ClientId"), ConfigClientId, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("ClientSecret"), ConfigClientSecret, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("ClientEncryptionKey"), ConfigEncryptionKey, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("OverrideCountryCode"), ConfigOverrideCountryCode, CONFIG_FILE);
						GConfig->GetString(CONFIG_SECTION_NAME, TEXT("OverrideLocaleCode"), ConfigOverrideLocaleCode, CONFIG_FILE);
						const FTCHARToUTF8 Utf8ProductId(*ConfigProductId);
						const FTCHARToUTF8 Utf8SandboxId(*ConfigSandboxId);
						const FTCHARToUTF8 Utf8DeploymentId(*ConfigDeploymentId);
						const FTCHARToUTF8 Utf8ClientId(*ConfigClientId);
						const FTCHARToUTF8 Utf8ClientSecret(*ConfigClientSecret);
						const FTCHARToUTF8 Utf8EncryptionKey(*ConfigEncryptionKey);
						const FTCHARToUTF8 Utf8OverrideCountryCode(*ConfigOverrideCountryCode);
						const FTCHARToUTF8 Utf8OverrideLocaleCode(*ConfigOverrideLocaleCode);
						EOS_Platform_Options PlatformOptions = { };
						PlatformOptions.ApiVersion = 14;
						UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_OPTIONS_API_LATEST, 15);
						PlatformOptions.Reserved = nullptr;
						PlatformOptions.SystemSpecificOptions = nullptr;
						PlatformOptions.ProductId = ConfigProductId.IsEmpty() ? nullptr : Utf8ProductId.Get();
						PlatformOptions.SandboxId = ConfigSandboxId.IsEmpty() ? nullptr : Utf8SandboxId.Get();
						PlatformOptions.ClientCredentials.ClientId = ConfigClientId.IsEmpty() ? nullptr : Utf8ClientId.Get();
						PlatformOptions.ClientCredentials.ClientSecret = ConfigClientSecret.IsEmpty() ? nullptr : Utf8ClientSecret.Get();
						PlatformOptions.bIsServer = false;
						PlatformOptions.EncryptionKey = ConfigEncryptionKey.IsEmpty() ? nullptr : Utf8EncryptionKey.Get();
						PlatformOptions.OverrideCountryCode = ConfigOverrideCountryCode.IsEmpty() ? nullptr : Utf8OverrideCountryCode.Get();
						PlatformOptions.OverrideLocaleCode = ConfigOverrideLocaleCode.IsEmpty() ? nullptr : Utf8OverrideLocaleCode.Get();
						PlatformOptions.DeploymentId = ConfigDeploymentId.IsEmpty() ? nullptr : Utf8DeploymentId.Get();
						PlatformOptions.Flags = EOS_PF_DISABLE_OVERLAY;
						PlatformOptions.CacheDirectory = nullptr;
						PlatformOptions.TickBudgetInMilliseconds = 1;
						PlatformOptions.IntegratedPlatformOptionsContainerHandle = nullptr;
						PlatformOptions.TaskNetworkTimeoutSeconds = nullptr;
#if UE_EDITOR
						//PlatformCreateOptions.Flags |= EOS_PF_LOADING_IN_EDITOR;
#endif
						EOS_Platform_RTCOptions PlatformRTCOptions = {};
						PlatformRTCOptions.ApiVersion = 3;
						UE_EOS_CHECK_API_MISMATCH(EOS_PLATFORM_RTCOPTIONS_API_LATEST, 3);
						PlatformRTCOptions.Reserved = nullptr;
						PlatformRTCOptions.BackgroundMode = EOS_ERTCBackgroundMode::EOS_RTCBM_KeepRoomsAlive;
						FString RTCBackgroundModeStr;
						if(GConfig->GetString(CONFIG_SECTION_NAME, TEXT("RTCBackgroundMode"), RTCBackgroundModeStr, CONFIG_FILE))
						{
							LexFromString(PlatformRTCOptions.BackgroundMode, *RTCBackgroundModeStr);					
						}
						PlatformOptions.RTCOptions = &PlatformRTCOptions;
						EosPlatformHandle = EOSPlatformCreate(PlatformOptions);

#if !UE_BUILD_SHIPPING
						InitializationPath = static_cast<uint8>(EEOSVoiceInitializationPath::LegacyInlineConfig);
#endif // !UE_BUILD_SHIPPING

					}

					if (!EosPlatformHandle)
					{
						UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Initialize CreatePlatform failed");
						Result = FVoiceChatResult(EVoiceChatResult::ImplementationError);
					}
				}
				else
				{
					UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Initialize SDKManager not initialized");
					Result = FVoiceChatResult(EVoiceChatResult::ImplementationError);
				}
			}

			if (Result.IsSuccess())
			{
				InitSession.EosRtcInterface = EOS_Platform_GetRTCInterface(*EosPlatformHandle);
				InitSession.EosLobbyInterface = EOS_Platform_GetLobbyInterface(*EosPlatformHandle);
				if (InitSession.EosRtcInterface && InitSession.EosLobbyInterface)
				{
					BindInitCallbacks();
					InitSession.State = EInitializationState::Initialized;
					PostInitialize();
					Result = FVoiceChatResult::CreateSuccess();
				}
				else
				{
					UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Initialize failed to get interface handles");
					Result = FVoiceChatResult(EVoiceChatResult::ImplementationError);
				}
			}
		}
		else
		{
			Result = VoiceChat::Errors::NotEnabled();
		}

		if (!Result.IsSuccess())
		{
			InitSession.Reset();
		}

		break;
	}
	case EInitializationState::Uninitializing:
		UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Initialize call unexpected while State=Uninitializing");
		Result = VoiceChat::Errors::InvalidState();
		break;
	case EInitializationState::Initializing:
		checkNoEntry(); // Should not be possible, Initialize is a synchronous call.
		UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Initialize call unexpected while State=Initializing");
		Result = VoiceChat::Errors::InvalidState();
		break;
	case EInitializationState::Initialized:
		Result = FVoiceChatResult::CreateSuccess();
		break;
	}

	InitCompleteDelegate.ExecuteIfBound(Result);
}

void FEOSVoiceChat::Uninitialize(const FOnVoiceChatUninitializeCompleteDelegate& UninitCompleteDelegate)
{
	switch (InitSession.State)
	{
	case EInitializationState::Uninitialized:
		UninitCompleteDelegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
		break;
	case EInitializationState::Uninitializing:
		InitSession.UninitializeCompleteDelegates.Emplace(UninitCompleteDelegate);
		break;
	case EInitializationState::Initializing:
		UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Uninitialize call unexpected while State=Initializing");
		UninitCompleteDelegate.ExecuteIfBound(VoiceChat::Errors::InvalidState());
		break;
	case EInitializationState::Initialized:
		InitSession.State = EInitializationState::Uninitializing;
		InitSession.UninitializeCompleteDelegates.Emplace(UninitCompleteDelegate);

		auto CompleteUninitialize = [this]()
		{
			PreUninitialize();
			UnbindInitCallbacks();

			const TArray<FOnVoiceChatUninitializeCompleteDelegate> UninitializeCompleteDelegates = MoveTemp(InitSession.UninitializeCompleteDelegates);
			InitSession.Reset();
			for (const FOnVoiceChatUninitializeCompleteDelegate& UninitializeCompleteDelegate : UninitializeCompleteDelegates)
			{
				UninitializeCompleteDelegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
			}
		};

		if (IsConnected())
		{
			Disconnect(FOnVoiceChatDisconnectCompleteDelegate::CreateLambda([this, CompleteUninitialize](const FVoiceChatResult& Result)
			{
				if (Result.IsSuccess())
				{
					CompleteUninitialize();
				}
				else
				{
					UE_LOGF(LogEOSVoiceChat, Warning, "FEOSVoiceChat::Uninitialize failed %ls", *LexToString(Result));

					InitSession.State = EInitializationState::Initialized;

					const TArray<FOnVoiceChatUninitializeCompleteDelegate> Delegates = MoveTemp(InitSession.UninitializeCompleteDelegates);
					for (const FOnVoiceChatUninitializeCompleteDelegate& Delegate : Delegates)
					{
						Delegate.ExecuteIfBound(Result);
					}
				}
			}));
		}
		else
		{
			CompleteUninitialize();
		}
		break;
	}
}

bool FEOSVoiceChat::IsInitialized() const
{
	return InitSession.State == EInitializationState::Initialized;
}

IVoiceChatUser* FEOSVoiceChat::CreateUser()
{
	const FEOSVoiceChatUserRef& User = VoiceChatUsers.Emplace_GetRef(MakeShared<FEOSVoiceChatUser, ESPMode::ThreadSafe>(*this));

	return &User.Get();
}

void FEOSVoiceChat::ReleaseUser(IVoiceChatUser* User)
{
	if (User)
	{		
		if (IsInitialized()
			&& IsConnected()
			&& User->IsLoggedIn())
		{
			UE_LOGF(LogEOSVoiceChat, Log, "ReleaseUser User=[%p] Logging out", User);
			User->Logout(FOnVoiceChatLogoutCompleteDelegate::CreateLambda([this, WeakThis = AsWeak(), User](const FString& PlayerName, const FVoiceChatResult& Result)
			{
				CHECKPIN();

				if (!Result.IsSuccess())
				{
					UE_LOGF(LogEOSVoiceChat, Warning, "ReleaseUser User=[%p] Logout failed, Result=[%ls]", User, *LexToString(Result))
				}

				ScheduleReleaseUser(User);
			}));
		}
		else
		{
			ScheduleReleaseUser(User);
		}
	}
}
#pragma endregion IVoiceChat

#pragma region IVoiceChatUser
void FEOSVoiceChat::SetSetting(const FString& Name, const FString& Value)
{
	GetVoiceChatUser().SetSetting(Name, Value);
}

FString FEOSVoiceChat::GetSetting(const FString& Name)
{
	return GetVoiceChatUser().GetSetting(Name);
}

void FEOSVoiceChat::SetAudioInputVolume(float Volume)
{
	GetVoiceChatUser().SetAudioInputVolume(Volume);
}

void FEOSVoiceChat::SetAudioOutputVolume(float Volume)
{
	GetVoiceChatUser().SetAudioOutputVolume(Volume);
}

float FEOSVoiceChat::GetAudioInputVolume() const
{
	return GetVoiceChatUser().GetAudioInputVolume();
}

float FEOSVoiceChat::GetAudioOutputVolume() const
{
	return GetVoiceChatUser().GetAudioOutputVolume();
}

void FEOSVoiceChat::SetAudioInputDeviceMuted(bool bIsMuted)
{
	GetVoiceChatUser().SetAudioInputDeviceMuted(bIsMuted);
}

void FEOSVoiceChat::SetAudioOutputDeviceMuted(bool bIsMuted)
{
	GetVoiceChatUser().SetAudioOutputDeviceMuted(bIsMuted);
}

bool FEOSVoiceChat::GetAudioInputDeviceMuted() const
{
	return GetVoiceChatUser().GetAudioInputDeviceMuted();
}

bool FEOSVoiceChat::GetAudioOutputDeviceMuted() const
{
	return GetVoiceChatUser().GetAudioOutputDeviceMuted();
}

TArray<FVoiceChatDeviceInfo> FEOSVoiceChat::GetAvailableInputDeviceInfos() const
{
	return GetVoiceChatUser().GetAvailableInputDeviceInfos();
}

TArray<FVoiceChatDeviceInfo> FEOSVoiceChat::GetAvailableOutputDeviceInfos() const
{
	return GetVoiceChatUser().GetAvailableOutputDeviceInfos();
}

FOnVoiceChatAvailableAudioDevicesChangedDelegate& FEOSVoiceChat::OnVoiceChatAvailableAudioDevicesChanged()
{
	return GetVoiceChatUser().OnVoiceChatAvailableAudioDevicesChanged();
}

void FEOSVoiceChat::SetInputDeviceId(const FString& InputDeviceId)
{
	GetVoiceChatUser().SetInputDeviceId(InputDeviceId);
}

void FEOSVoiceChat::SetOutputDeviceId(const FString& OutputDeviceId)
{
	GetVoiceChatUser().SetOutputDeviceId(OutputDeviceId);
}

FVoiceChatDeviceInfo FEOSVoiceChat::GetInputDeviceInfo() const
{
	return GetVoiceChatUser().GetInputDeviceInfo();
}

FVoiceChatDeviceInfo FEOSVoiceChat::GetOutputDeviceInfo() const
{
	return GetVoiceChatUser().GetOutputDeviceInfo();
}

FVoiceChatDeviceInfo FEOSVoiceChat::GetDefaultInputDeviceInfo() const
{
	return GetVoiceChatUser().GetDefaultInputDeviceInfo();
}

FVoiceChatDeviceInfo FEOSVoiceChat::GetDefaultOutputDeviceInfo() const
{
	return GetVoiceChatUser().GetDefaultOutputDeviceInfo();
}

void FEOSVoiceChat::Connect(const FOnVoiceChatConnectCompleteDelegate& Delegate)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();

	if (!IsInitialized())
	{
		Result = VoiceChat::Errors::NotInitialized();
	}
	else if (ConnectionState == EConnectionState::Disconnecting)
	{
		Result = VoiceChat::Errors::DisconnectInProgress();
	}

	if (!Result.IsSuccess())
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "Connect %ls", *LexToString(Result));
		Delegate.ExecuteIfBound(Result);
	}
	else if (IsConnected())
	{
		Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
	}
	else
	{
		ConnectionState = EConnectionState::Connected;
		Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
		OnVoiceChatConnected().Broadcast();
	}
}

void FEOSVoiceChat::Disconnect(const FOnVoiceChatDisconnectCompleteDelegate& Delegate)
{
	// TODO Handle Disconnecting / Connecting states now this is async.
	if (IsConnected())
	{
		ConnectionState = EConnectionState::Disconnecting;

		TSet<FEOSVoiceChatUser*> UsersToLogout;

		if (SingleUserVoiceChatUser)
		{
			FEOSVoiceChatUser::ELoginState LoginState = SingleUserVoiceChatUser->LoginSession.State;
			if (LoginState == FEOSVoiceChatUser::ELoginState::LoggedIn || LoginState == FEOSVoiceChatUser::ELoginState::LoggingOut)
			{
				UsersToLogout.Emplace(SingleUserVoiceChatUser);
			}
		}
		else
		{
			for (const FEOSVoiceChatUserRef& VoiceChatUser : VoiceChatUsers)
			{
				FEOSVoiceChatUser::ELoginState LoginState = VoiceChatUser->LoginSession.State;
				if (LoginState == FEOSVoiceChatUser::ELoginState::LoggedIn || LoginState == FEOSVoiceChatUser::ELoginState::LoggingOut)
				{
					UsersToLogout.Emplace(&VoiceChatUser.Get());
				}
			}
		}

		if (UsersToLogout.Num() > 0)
		{
			struct FEOSVoiceChatDisconnectState
			{
				FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
				FOnVoiceChatDisconnectCompleteDelegate CompletionDelegate;
				int32 UsersToLogoutCount;
			};
			TSharedPtr<FEOSVoiceChatDisconnectState> DisconnectState = MakeShared<FEOSVoiceChatDisconnectState>();
			DisconnectState->UsersToLogoutCount = UsersToLogout.Num();
			DisconnectState->CompletionDelegate = Delegate;

			for (FEOSVoiceChatUser* User : UsersToLogout)
			{
				User->LogoutInternal(FOnVoiceChatLogoutCompleteDelegate::CreateLambda([this, User, DisconnectState](const FString& PlayerName, const FVoiceChatResult& PlayerResult)
				{
					if (!PlayerResult.IsSuccess())
					{
						UE_LOGF(LogEOSVoiceChat, Warning, "Disconnect LogoutCompleteDelegate PlayerName=[%ls] Result=%ls", *PlayerName, *LexToString(PlayerResult));
						DisconnectState->Result = PlayerResult;
					}

					DisconnectState->UsersToLogoutCount--;

					if (DisconnectState->UsersToLogoutCount == 0)
					{
						ConnectionState = DisconnectState->Result.IsSuccess() ? EConnectionState::Disconnected : EConnectionState::Connected;
						DisconnectState->CompletionDelegate.ExecuteIfBound(DisconnectState->Result);
						if (ConnectionState == EConnectionState::Disconnected)
						{
							OnVoiceChatDisconnected().Broadcast(DisconnectState->Result);
						}
					}
				}));
			}
		}
		else
		{
			ConnectionState = EConnectionState::Disconnected;
			Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
			OnVoiceChatDisconnected().Broadcast(FVoiceChatResult::CreateSuccess());
		}
	}
	else
	{
		Delegate.ExecuteIfBound(FVoiceChatResult::CreateSuccess());
	}
}

bool FEOSVoiceChat::IsConnecting() const
{
	return false;
}

bool FEOSVoiceChat::IsConnected() const
{
	return ConnectionState == EConnectionState::Connected;
}

void FEOSVoiceChat::Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& Delegate)
{
	GetVoiceChatUser().Login(PlatformId, PlayerName, Credentials, Delegate);
}

void FEOSVoiceChat::Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate)
{
	GetVoiceChatUser().Logout(Delegate);
}

bool FEOSVoiceChat::IsLoggingIn() const
{
	return GetVoiceChatUser().IsLoggingIn();
}

bool FEOSVoiceChat::IsLoggedIn() const
{
	return GetVoiceChatUser().IsLoggedIn();
}

FOnVoiceChatLoggedInDelegate& FEOSVoiceChat::OnVoiceChatLoggedIn()
{
	return GetVoiceChatUser().OnVoiceChatLoggedIn();
}

FOnVoiceChatLoggedOutDelegate& FEOSVoiceChat::OnVoiceChatLoggedOut()
{
	return GetVoiceChatUser().OnVoiceChatLoggedOut();
}

FString FEOSVoiceChat::GetLoggedInPlayerName() const
{
	return GetVoiceChatUser().GetLoggedInPlayerName();
}

void FEOSVoiceChat::BlockPlayers(const TArray<FString>& PlayerNames)
{
	GetVoiceChatUser().BlockPlayers(PlayerNames);
}

void FEOSVoiceChat::UnblockPlayers(const TArray<FString>& PlayerNames)
{
	GetVoiceChatUser().UnblockPlayers(PlayerNames);
}

void FEOSVoiceChat::JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties)
{
	GetVoiceChatUser().JoinChannel(ChannelName, ChannelCredentials, ChannelType, Delegate, Channel3dProperties);
}

void FEOSVoiceChat::LeaveChannel(const FString& Channel, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate)
{
	GetVoiceChatUser().LeaveChannel(Channel, Delegate);
}

FOnVoiceChatChannelJoinedDelegate& FEOSVoiceChat::OnVoiceChatChannelJoined()
{
	return GetVoiceChatUser().OnVoiceChatChannelJoined();
}

FOnVoiceChatChannelExitedDelegate& FEOSVoiceChat::OnVoiceChatChannelExited()
{
	return GetVoiceChatUser().OnVoiceChatChannelExited();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOnVoiceChatCallStatsUpdatedDelegate& FEOSVoiceChat::OnVoiceChatCallStatsUpdated()
{
	return GetVoiceChatUser().OnVoiceChatCallStatsUpdated();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FOnVoiceChatCallStatsUpdatedDelegate2& FEOSVoiceChat::OnVoiceChatCallStatsUpdated2()
{
	return GetVoiceChatUser().OnVoiceChatCallStatsUpdated2();
}

TOptional<FVoiceChatCallStats> FEOSVoiceChat::GetChannelCallStats(const FString& ChannelName) const
{
	return GetVoiceChatUser().GetChannelCallStats(ChannelName);
}

void FEOSVoiceChat::Set3DPosition(const FString& ChannelName, const FVector& Position)
{
	GetVoiceChatUser().Set3DPosition(ChannelName, Position);
}

TArray<FString> FEOSVoiceChat::GetChannels() const
{
	return GetVoiceChatUser().GetChannels();
}

TArray<FString> FEOSVoiceChat::GetPlayersInChannel(const FString& ChannelName) const
{
	return GetVoiceChatUser().GetPlayersInChannel(ChannelName);
}

EVoiceChatChannelType FEOSVoiceChat::GetChannelType(const FString& ChannelName) const
{
	return GetVoiceChatUser().GetChannelType(ChannelName);
}

FOnVoiceChatPlayerAddedDelegate& FEOSVoiceChat::OnVoiceChatPlayerAdded()
{
	return GetVoiceChatUser().OnVoiceChatPlayerAdded();
}

FOnVoiceChatPlayerRemovedDelegate& FEOSVoiceChat::OnVoiceChatPlayerRemoved()
{
	return GetVoiceChatUser().OnVoiceChatPlayerRemoved();
}

bool FEOSVoiceChat::IsPlayerTalking(const FString& PlayerName) const
{
	return GetVoiceChatUser().IsPlayerTalking(PlayerName);
}

FOnVoiceChatPlayerTalkingUpdatedDelegate& FEOSVoiceChat::OnVoiceChatPlayerTalkingUpdated()
{
	return GetVoiceChatUser().OnVoiceChatPlayerTalkingUpdated();
}

void FEOSVoiceChat::SetPlayerMuted(const FString& PlayerName, bool bMuted)
{
	GetVoiceChatUser().SetPlayerMuted(PlayerName, bMuted);
}

bool FEOSVoiceChat::IsPlayerMuted(const FString& PlayerName) const
{
	return GetVoiceChatUser().IsPlayerMuted(PlayerName);
}

void FEOSVoiceChat::SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bMuted)
{
	GetVoiceChatUser().SetChannelPlayerMuted(ChannelName, PlayerName, bMuted);
}

bool FEOSVoiceChat::IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const
{
	return GetVoiceChatUser().IsChannelPlayerMuted(ChannelName, PlayerName);
}

FOnVoiceChatPlayerMuteUpdatedDelegate& FEOSVoiceChat::OnVoiceChatPlayerMuteUpdated()
{
	return GetVoiceChatUser().OnVoiceChatPlayerMuteUpdated();
}

void FEOSVoiceChat::SetPlayerVolume(const FString& PlayerName, float Volume)
{
	GetVoiceChatUser().SetPlayerVolume(PlayerName, Volume);
}

float FEOSVoiceChat::GetPlayerVolume(const FString& PlayerName) const
{
	return GetVoiceChatUser().GetPlayerVolume(PlayerName);
}

FOnVoiceChatPlayerVolumeUpdatedDelegate& FEOSVoiceChat::OnVoiceChatPlayerVolumeUpdated()
{
	return GetVoiceChatUser().OnVoiceChatPlayerVolumeUpdated();
}

void FEOSVoiceChat::TransmitToAllChannels()
{
	GetVoiceChatUser().TransmitToAllChannels();
}

void FEOSVoiceChat::TransmitToNoChannels()
{
	GetVoiceChatUser().TransmitToNoChannels();
}

void FEOSVoiceChat::TransmitToSpecificChannels(const TSet<FString>& ChannelNames)
{
	GetVoiceChatUser().TransmitToSpecificChannels(ChannelNames);
}

EVoiceChatTransmitMode FEOSVoiceChat::GetTransmitMode() const
{
	return GetVoiceChatUser().GetTransmitMode();
}

TSet<FString> FEOSVoiceChat::GetTransmitChannels() const
{
	return GetVoiceChatUser().GetTransmitChannels();
}

FDelegateHandle FEOSVoiceChat::StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate)
{
	return GetVoiceChatUser().StartRecording(Delegate);
}

void FEOSVoiceChat::StopRecording(FDelegateHandle Handle)
{
	GetVoiceChatUser().StopRecording(Handle);
}

FDelegateHandle FEOSVoiceChat::RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate)
{
	return GetVoiceChatUser().RegisterOnVoiceChatAfterCaptureAudioReadDelegate(Delegate);
}

void FEOSVoiceChat::UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle)
{
	GetVoiceChatUser().UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(Handle);
}

FDelegateHandle FEOSVoiceChat::RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate)
{
	return GetVoiceChatUser().RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(Delegate);
}

void FEOSVoiceChat::UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle)
{
	GetVoiceChatUser().UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(Handle);
}

FDelegateHandle FEOSVoiceChat::RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate)
{
	return GetVoiceChatUser().RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Delegate);
}

void FEOSVoiceChat::UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle)
{
	GetVoiceChatUser().UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(Handle);
}

FDelegateHandle FEOSVoiceChat::RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate)
{
	return GetVoiceChatUser().RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(Delegate);
}

void FEOSVoiceChat::UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle)
{
	GetVoiceChatUser().UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(Handle);
}

FString FEOSVoiceChat::InsecureGetLoginToken(const FString& PlayerName)
{
	return GetVoiceChatUser().InsecureGetLoginToken(PlayerName);
}

FString FEOSVoiceChat::InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties)
{
	return GetVoiceChatUser().InsecureGetJoinToken(ChannelName, ChannelType, Channel3dProperties);
}
#pragma endregion IVoiceChatUser

void FEOSVoiceChat::BindInitCallbacks()
{
	EOS_RTCAudio_AddNotifyAudioDevicesChangedOptions AudioDevicesChangedOptions = {};
	AudioDevicesChangedOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYAUDIODEVICESCHANGED_API_LATEST, 1);
	InitSession.OnAudioDevicesChangedNotificationId = EOS_RTCAudio_AddNotifyAudioDevicesChanged(EOS_RTC_GetAudioInterface(GetRtcInterface()), &AudioDevicesChangedOptions, this, &FEOSVoiceChat::OnAudioDevicesChangedStatic);
	if (InitSession.OnAudioDevicesChangedNotificationId == EOS_INVALID_NOTIFICATIONID)
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "BindInitCallbacks EOS_RTC_AddNotifyAudioDevicesChanged failed");
	}

	OnAudioDevicesChanged();

	FCoreDelegates::TSOnConfigSectionsChanged().AddThreadSafeSP(this, &FEOSVoiceChat::OnConfigSectionsChanged);
	LoadConfig();
}

void FEOSVoiceChat::UnbindInitCallbacks()
{
	if (InitSession.OnAudioDevicesChangedNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyAudioDevicesChanged(EOS_RTC_GetAudioInterface(GetRtcInterface()), InitSession.OnAudioDevicesChangedNotificationId);
		InitSession.OnAudioDevicesChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	}

	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
}

void FEOSVoiceChat::OnAudioDevicesChangedStatic(const EOS_RTCAudio_AudioDevicesChangedCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChat* EosVoiceChatPtr = static_cast<FEOSVoiceChat*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnAudioDevicesChanged();
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnAudioDevicesChangedStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnAudioDevicesChangedStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChat::OnAudioDevicesChanged()
{
	InitSession.EosAudioDevicePool->RefreshAudioDevices(FEOSAudioDevicePool::FOnAudioDevicePoolRefreshAudioDevicesCompleteDelegate::CreateLambda([WeakThis = AsWeak()](const FVoiceChatResult& Result) -> void
	{
		CHECKPIN();

		if (!Result.IsSuccess())
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnAudioDevicesChanged RefreshAudioDevicesCompletionDelegate failed, Result=[%ls]", *LexToString(Result));
		}

		StrongThis->OnVoiceChatAvailableAudioDevicesChangedDelegate.Broadcast();
	}));
}

void FEOSVoiceChat::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == CONFIG_FILE)
	{
		LoadConfig();
	}
}

void FEOSVoiceChat::LoadConfig()
{
	// Helper lambda to apply a single RTC setting
	auto ApplyRTCSetting = [this](const TCHAR* SettingName, const TCHAR* SettingValue)
	{
		const FTCHARToUTF8 Utf8Key(SettingName);
		const FTCHARToUTF8 Utf8Value(SettingValue);

		EOS_RTC_SetSettingOptions SetSettingOptions = { };
		SetSettingOptions.ApiVersion = EOS_RTC_SETSETTING_API_LATEST;
		static_assert(EOS_RTC_SETSETTING_API_LATEST == 1, "EOS_RTC_SetSettingOptions updated, check new fields");
		SetSettingOptions.SettingName = Utf8Key.Get();
		SetSettingOptions.SettingValue = Utf8Value.Get();
		const EOS_EResult Result = EOS_RTC_SetSetting(GetRtcInterface(), &SetSettingOptions);
		UE_CLOGF(Result != EOS_EResult::EOS_Success, LogEOSVoiceChat, Warning,
			"SetSetting Key=%ls Value=%ls Result=%ls", SettingName, SettingValue, *LexToString(Result));
	};

	// Legacy config load: RTCSetting_<SettingName>=<Value>
	auto SetFromConfig = [this, &ApplyRTCSetting](const TCHAR* InSettingName)
	{
		const FString ConfigKey(FString::Printf(TEXT("RTCSetting_%s"), InSettingName));
		FString ConfigValue;
		if (GConfig->GetString(CONFIG_SECTION_NAME, *ConfigKey, ConfigValue, CONFIG_FILE))
		{
			ApplyRTCSetting(InSettingName, *ConfigValue);
		}
	};
	SetFromConfig(TEXT("MaxTickInterval"));

	// New config load: Get RTCSettings array with Key=Value entries to apply
	TArray<FString> RTCSettings;
	GConfig->GetArray(CONFIG_SECTION_NAME, TEXT("RTCSettings"), RTCSettings, CONFIG_FILE);
	for (const FString& RTCSetting : RTCSettings)
	{
		TArray<FString> Split;
		RTCSetting.ParseIntoArray(Split, TEXT("="));
		UE_CLOGF(Split.Num() != 2, LogEOSVoiceChat, Warning, "LoadConfig Cannot parse [%ls]", *RTCSetting);

		if (Split.Num() == 2)
		{
			ApplyRTCSetting(*Split[0], *Split[1]);
		}
	}
}

void FEOSVoiceChat::ScheduleReleaseUser(IVoiceChatUser* User)
{
	FEOSVoiceChatUserRef UserRef = static_cast<FEOSVoiceChatUser*>(User)->AsShared();
	ReleasedVoiceChatUsers.Add(UserRef);
	VoiceChatUsers.Remove(UserRef);

	AsyncTask(ENamedThreads::GameThread, [this, WeakThis = AsWeak()]()
	{
		CHECKPIN();

		while (ReleasedVoiceChatUsers.Num() > 0)
		{
			UE_LOGF(LogEOSVoiceChat, Log, "Releasing User=[%p]", &ReleasedVoiceChatUsers[0].Get());
			ReleasedVoiceChatUsers.RemoveAtSwap(0);
		}
	});
}

FEOSVoiceChatUser& FEOSVoiceChat::GetVoiceChatUser()
{
	if (!SingleUserVoiceChatUser)
	{
		SingleUserVoiceChatUser = static_cast<FEOSVoiceChatUser*>(CreateUser());
		ensureMsgf(VoiceChatUsers.Num() == 1, TEXT("When using multiple users, all connections should be managed by an IVoiceChatUser"));
	}

	return *SingleUserVoiceChatUser;
}

FEOSVoiceChatUser& FEOSVoiceChat::GetVoiceChatUser() const
{
	return const_cast<FEOSVoiceChat*>(this)->GetVoiceChatUser();
}

bool FEOSVoiceChat::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !NO_LOGGING
#define EOS_EXEC_LOG(Fmt, ...) Ar.CategorizedLogf(LogEOSVoiceChat.GetCategoryName(), ELogVerbosity::Log, Fmt, ##__VA_ARGS__)
#else
#define EOS_EXEC_LOG(Fmt, ...) 
#endif

	if (FParse::Command(&Cmd, CONFIG_SECTION_NAME))
	{
		const TCHAR* SubCmd = Cmd;
		if (FParse::Command(&Cmd, TEXT("LIST")))
		{
			EOS_EXEC_LOG(TEXT("InstanceId=%d Users=%d"), InstanceId, VoiceChatUsers.Num());
			if (VoiceChatUsers.Num() > 0)
			{
				for (int UserIndex = 0; UserIndex < VoiceChatUsers.Num(); ++UserIndex)
				{
					const FEOSVoiceChatUserRef& User = VoiceChatUsers[UserIndex];
					EOS_EXEC_LOG(TEXT("  EOSUser Index:%i PlayerName:%s"), UserIndex, *User->GetLoggedInPlayerName());
				}
			}
			return true;
		}

		int64 InstanceIdParam = 0;
		FParse::Value(Cmd, TEXT("InstanceId="), InstanceIdParam);
		if (InstanceIdParam == InstanceId)
		{
			if (FParse::Command(&Cmd, TEXT("INFO")))
			{
				EOS_EXEC_LOG(TEXT("Initialized: %s"), *LexToString(IsInitialized()));

#if !UE_BUILD_SHIPPING
				EOS_EXEC_LOG(TEXT("Initialization Path: %s"), LexToString(static_cast<EEOSVoiceInitializationPath>(InitializationPath)));
				if (OverridePlatformConfigName.IsSet())
				{
					EOS_EXEC_LOG(TEXT("OverridePlatformConfigName: %s"), *OverridePlatformConfigName.GetValue());
				}
#endif // !UE_BUILD_SHIPPING

				if (IsInitialized())
				{
					EOS_EXEC_LOG(TEXT("Connection Status: %s"), LexToString(ConnectionState));

					for (int UserIndex = 0; UserIndex < VoiceChatUsers.Num(); ++UserIndex)
					{
						const FEOSVoiceChatUserRef& User = VoiceChatUsers[UserIndex];
						EOS_EXEC_LOG(TEXT("  User Index:%i PlayerName:%s"), UserIndex, *User->GetLoggedInPlayerName());
						User->Exec(InWorld, SubCmd, Ar);
					}
				}
				return true;
			}
	#if !UE_BUILD_SHIPPING
			else if (FParse::Command(&Cmd, TEXT("INITIALIZE")))
			{
				Initialize(FOnVoiceChatInitializeCompleteDelegate::CreateLambda([this](const FVoiceChatResult& Result)
				{
					UE_LOGF(LogEOSVoiceChat, Display, "EOS INITIALIZE success:%ls, Initialization Path: %ls", *LexToString(Result), LexToString(static_cast<EEOSVoiceInitializationPath>(InitializationPath)));
					if (OverridePlatformConfigName.IsSet())
					{
						UE_LOGF(LogEOSVoiceChat, Display, "OverridePlatformConfigName: %ls", *OverridePlatformConfigName.GetValue());
					}
				}));
				return true;
			}
			else if (FParse::Command(&Cmd, TEXT("UNINITIALIZE")))
			{
				Uninitialize(FOnVoiceChatUninitializeCompleteDelegate::CreateLambda([](const FVoiceChatResult& Result)
				{
					UE_LOGF(LogEOSVoiceChat, Display, "EOS UNINITIALIZE success:%ls", *LexToString(Result));
				}));
				return true;
			}
			else if (FParse::Command(&Cmd, TEXT("CONNECT")))
			{
				Connect(FOnVoiceChatConnectCompleteDelegate::CreateLambda([](const FVoiceChatResult& Result)
				{
					UE_LOGF(LogEOSVoiceChat, Display, "EOS CONNECT result:%ls", *LexToString(Result));
				}));
				return true;
			}
			else if (FParse::Command(&Cmd, TEXT("DISCONNECT")))
			{
				Disconnect(FOnVoiceChatDisconnectCompleteDelegate::CreateLambda([](const FVoiceChatResult& Result)
				{
					UE_LOGF(LogEOSVoiceChat, Display, "EOS DISCONNECT result:%ls", *LexToString(Result));
				}));
				return true;
			}
			else if (FParse::Command(&Cmd, TEXT("CREATEUSER")))
			{
				if (!SingleUserVoiceChatUser)
				{
					UsersCreatedByConsoleCommand.Add(CreateUser());
					EOS_EXEC_LOG(TEXT("EOS CREATEUSER success"));
					return true;
				}
				else
				{
					EOS_EXEC_LOG(TEXT("EOS CREATEUSER failed, single user set."));
					return true;
				}
			}
			else if (FParse::Command(&Cmd, TEXT("CREATESINGLEUSER")))
			{
				if (SingleUserVoiceChatUser)
				{
					EOS_EXEC_LOG(TEXT("EOS CREATESINGLEUSER already exists"));
					return true;
				}
				else if (VoiceChatUsers.Num() == 0)
				{
					GetVoiceChatUser();
					EOS_EXEC_LOG(TEXT("EOS CREATESINGLEUSER success"));
					return true;
				}
				else
				{
					EOS_EXEC_LOG(TEXT("EOS CREATESINGLEUSER failed, VoiceChatUsers not empty."));
					return true;
				}
			}
			else
			{	
				int UserIndex = 0;
				if (FParse::Value(Cmd, TEXT("UserIndex="), UserIndex))
				{
					if (UserIndex < VoiceChatUsers.Num())
					{
						const FEOSVoiceChatUserRef& UserRef = VoiceChatUsers[UserIndex];
						if (FParse::Command(&Cmd, TEXT("RELEASEUSER")))
						{
							IVoiceChatUser* User = &UserRef.Get();
							if (UsersCreatedByConsoleCommand.RemoveSwap(User))
							{
								EOS_EXEC_LOG(TEXT("EOS RELEASEUSER releasing UserIndex=%d..."), UserIndex);
								ReleaseUser(User);
							}
							else
							{
								EOS_EXEC_LOG(TEXT("EOS RELEASEUSER UserIndex=%d not created by CREATEUSER call."), UserIndex);
							}
							return true;
						}
						else
						{
							return UserRef->Exec(InWorld, Cmd, Ar);
						}
					}
					else
					{
						EOS_EXEC_LOG(TEXT("EOS RELEASEUSER UserIndex=%d not found, VoiceChatUsers.Num=%d"), UserIndex, VoiceChatUsers.Num());
						return true;
					}
				}
				else if (SingleUserVoiceChatUser)
				{
					return SingleUserVoiceChatUser->Exec(InWorld, SubCmd, Ar);
				}
				else
				{
					EOS_EXEC_LOG(TEXT("EOS User index not specified, and no single user created. Either CREATEUSER and specify UserIndex=n in subsequent commands, or CREATESINGLEUSER (no UserIndex=n necessary in subsequent commands)"));
					return true;
				}
			}
#endif // !UE_BUILD_SHIPPING
		}
	}

#undef EOS_EXEC_LOG

	return false;
}

FEOSVoiceChat::FInitSession::FInitSession()
	: EosAudioDevicePool{ MakeShared<FEOSAudioDevicePool>(EosRtcInterface) }
{
}

void FEOSVoiceChat::FInitSession::Reset()
{
	State = EInitializationState::Uninitialized;

	UninitializeCompleteDelegates = TArray<FOnVoiceChatUninitializeCompleteDelegate>{};

	EosRtcInterface = nullptr;
	EosLobbyInterface = nullptr;

	OnAudioDevicesChangedNotificationId = EOS_INVALID_NOTIFICATIONID;

	EosAudioDevicePool = MakeShared<FEOSAudioDevicePool>(EosRtcInterface);
}

IEOSPlatformHandlePtr FEOSVoiceChat::EOSPlatformCreate(EOS_Platform_Options& PlatformOptions)
{
	return SDKManager.CreatePlatform(PlatformOptions);
}

const TCHAR* LexToString(FEOSVoiceChat::EConnectionState State)
{
	switch (State)
	{
	case FEOSVoiceChat::EConnectionState::Disconnected:		return TEXT("Disconnected");
	case FEOSVoiceChat::EConnectionState::Disconnecting:	return TEXT("Disconnecting");
	case FEOSVoiceChat::EConnectionState::Connecting:		return TEXT("Connecting");
	case FEOSVoiceChat::EConnectionState::Connected:		return TEXT("Connected");
	default:												return TEXT("Unknown");
	}
}

#undef CHECKPIN
#undef CONFIG_FILE
#undef CONFIG_SECTION_NAME

#endif // WITH_EOSVOICECHAT