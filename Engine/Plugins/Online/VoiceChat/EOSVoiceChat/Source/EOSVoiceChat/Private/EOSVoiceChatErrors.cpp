// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVoiceChatErrors.h"

#if WITH_EOSVOICECHAT

#include "VoiceChatResult.h"
#include "VoiceChatErrors.h"

#include "EOSShared.h"
#include "EOSVoiceChatErrors.h"

#include "eos_common.h"

FVoiceChatResult ResultFromEOSResult(const EOS_EResult EosResult)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
	if (EosResult != EOS_EResult::EOS_Success)
	{
		switch (EosResult)
		{
		case EOS_EResult::EOS_InvalidCredentials:
		case EOS_EResult::EOS_InvalidAuth:
		case EOS_EResult::EOS_Token_Not_Account:
			Result = VoiceChat::Errors::CredentialsInvalid();
			break;
		case EOS_EResult::EOS_InvalidUser:
		case EOS_EResult::EOS_InvalidParameters:
		case EOS_EResult::EOS_LimitExceeded:
			Result = VoiceChat::Errors::InvalidArgument();
			break;
		case EOS_EResult::EOS_AccessDenied:
		case EOS_EResult::EOS_MissingPermissions:
		case EOS_EResult::EOS_InvalidRequest:
			Result = VoiceChat::Errors::NotPermitted();
			break;
		case EOS_EResult::EOS_TooManyRequests:
			Result = VoiceChat::Errors::Throttled();
			break;
		case EOS_EResult::EOS_AlreadyPending:
			Result = VoiceChat::Errors::AlreadyInProgress();
			break;
		case EOS_EResult::EOS_NotConfigured:
			Result = VoiceChat::Errors::MissingConfig();
			break;
		case EOS_EResult::EOS_AlreadyConfigured:
			Result = VoiceChat::Errors::InvalidState();
			break;

			// TODO the rest
		case EOS_EResult::EOS_OperationWillRetry:
		case EOS_EResult::EOS_NoChange:
		case EOS_EResult::EOS_VersionMismatch:
		case EOS_EResult::EOS_Disabled:
		case EOS_EResult::EOS_DuplicateNotAllowed:

			// Auth/Presence/Friends/Ecom we're not expecting to encounter
		case EOS_EResult::EOS_Auth_AccountLocked:
		case EOS_EResult::EOS_Auth_AccountLockedForUpdate:
		case EOS_EResult::EOS_Auth_InvalidRefreshToken:
		case EOS_EResult::EOS_Auth_InvalidToken:
		case EOS_EResult::EOS_Auth_AuthenticationFailure:
		case EOS_EResult::EOS_Auth_InvalidPlatformToken:
		case EOS_EResult::EOS_Auth_WrongAccount:
		case EOS_EResult::EOS_Auth_WrongClient:
		case EOS_EResult::EOS_Auth_FullAccountRequired:
		case EOS_EResult::EOS_Auth_HeadlessAccountRequired:
		case EOS_EResult::EOS_Auth_PasswordResetRequired:
		case EOS_EResult::EOS_Auth_PasswordCannotBeReused:
		case EOS_EResult::EOS_Auth_Expired:
		case EOS_EResult::EOS_Auth_PinGrantCode:
		case EOS_EResult::EOS_Auth_PinGrantExpired:
		case EOS_EResult::EOS_Auth_PinGrantPending:
		case EOS_EResult::EOS_Auth_ExternalAuthNotLinked:
		case EOS_EResult::EOS_Auth_ExternalAuthRevoked:
		case EOS_EResult::EOS_Auth_ExternalAuthInvalid:
		case EOS_EResult::EOS_Auth_ExternalAuthRestricted:
		case EOS_EResult::EOS_Auth_ExternalAuthCannotLogin:
		case EOS_EResult::EOS_Auth_ExternalAuthExpired:
		case EOS_EResult::EOS_Auth_ExternalAuthIsLastLoginType:
		case EOS_EResult::EOS_Auth_ExchangeCodeNotFound:
		case EOS_EResult::EOS_Auth_OriginatingExchangeCodeSessionExpired:
		case EOS_EResult::EOS_Auth_AccountNotActive:
		case EOS_EResult::EOS_Auth_MFARequired:
		case EOS_EResult::EOS_Auth_ParentalControls:
		case EOS_EResult::EOS_Auth_NoRealId:
		case EOS_EResult::EOS_Friends_InviteAwaitingAcceptance:
		case EOS_EResult::EOS_Friends_NoInvitation:
		case EOS_EResult::EOS_Friends_AlreadyFriends:
		case EOS_EResult::EOS_Friends_NotFriends:
		case EOS_EResult::EOS_Presence_DataInvalid:
		case EOS_EResult::EOS_Presence_DataLengthInvalid:
		case EOS_EResult::EOS_Presence_DataKeyInvalid:
		case EOS_EResult::EOS_Presence_DataKeyLengthInvalid:
		case EOS_EResult::EOS_Presence_DataValueInvalid:
		case EOS_EResult::EOS_Presence_DataValueLengthInvalid:
		case EOS_EResult::EOS_Presence_RichTextInvalid:
		case EOS_EResult::EOS_Presence_RichTextLengthInvalid:
		case EOS_EResult::EOS_Presence_StatusInvalid:
		case EOS_EResult::EOS_Ecom_EntitlementStale:

			// Intentional fall-through cases
		case EOS_EResult::EOS_NoConnection:
		case EOS_EResult::EOS_Canceled:
		case EOS_EResult::EOS_IncompatibleVersion:
		case EOS_EResult::EOS_UnrecognizedResponse:
		case EOS_EResult::EOS_NotImplemented:
		case EOS_EResult::EOS_NotFound:
		case EOS_EResult::EOS_UnexpectedError:
		default:
			// TODO map more EOS statuses to text error codes
			Result = EOSVOICECHAT_ERROR(EVoiceChatResult::ImplementationError, *LexToString(EosResult));
			break;
		}

		Result.ErrorNum = static_cast<int>(EosResult);
		Result.ErrorDesc = FString::Printf(TEXT("EOS_EResult=%s"), *LexToString(EosResult));
	}

	return Result;
}

#endif // WITH_EOSVOICECHAT
