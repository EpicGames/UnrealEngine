// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

#include "VoiceChatResult.h"

#define EOSVOICECHAT_ERROR(...) FVoiceChatResult::CreateError(TEXT("errors.com.epicgames.voicechat.eos"), __VA_ARGS__)

enum class EOS_EResult : int32_t;

FVoiceChatResult ResultFromEOSResult(const EOS_EResult EosResult);

#endif // WITH_EOSVOICECHAT
