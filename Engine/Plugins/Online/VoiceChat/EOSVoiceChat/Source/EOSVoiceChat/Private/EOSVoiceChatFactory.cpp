// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVoiceChatFactory.h"

#if WITH_EOSVOICECHAT

#include "Misc/Parse.h"
#include "IEOSSDKManager.h"
#include "EOSVoiceChatLog.h"

#include COMPILED_PLATFORM_HEADER(EOSVoiceChat.h)

IVoiceChatPtr FEOSVoiceChatFactory::CreateInstance()
{
	IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get();
	check(EOSSDKManager);

	FEOSVoiceChatPtr VoiceChat = MakeShared<FPlatformEOSVoiceChat, ESPMode::ThreadSafe>(*EOSSDKManager);
	Instances.Emplace(VoiceChat);
	return VoiceChat;
}

IVoiceChatPtr FEOSVoiceChatFactory::CreateInstanceWithPlatform(const IEOSPlatformHandlePtr& PlatformHandle)
{
	IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get();
	check(EOSSDKManager);

	FEOSVoiceChatPtr VoiceChat = MakeShared<FPlatformEOSVoiceChat, ESPMode::ThreadSafe>(*EOSSDKManager);
	VoiceChat->SetPlatformHandle(PlatformHandle);
	Instances.Emplace(VoiceChat);

	return VoiceChat;
}

IVoiceChatPtr FEOSVoiceChatFactory::CreateInstanceWithPlatformConfig(const FString& PlatformConfigName)
{
	IEOSSDKManager* EOSSDKManager = IEOSSDKManager::Get();
	check(EOSSDKManager);

	FEOSVoiceChatPtr VoiceChat = MakeShared<FPlatformEOSVoiceChat, ESPMode::ThreadSafe>(*EOSSDKManager);
	VoiceChat->SetPlatformConfigName(PlatformConfigName);
	Instances.Emplace(VoiceChat);

	return VoiceChat;
}

#if UE_ALLOW_EXEC_COMMANDS
bool FEOSVoiceChatFactory::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// When list is requested, the factory is responsible for returning true when all instances have reported their state.
	bool bListRequested = false;

	const TCHAR* CmdCopy = Cmd;
	if (FParse::Command(&CmdCopy, TEXT("EOSVOICECHAT")))
	{
		if (FParse::Command(&CmdCopy, TEXT("LIST")))
		{
			bListRequested = true;
		}
	}

	for (TArray<IVoiceChatWeakPtr>::TIterator Iter = Instances.CreateIterator(); Iter; ++Iter)
	{
		if (IVoiceChatPtr StrongPtr = Iter->Pin())
		{
			FEOSVoiceChat& EosVoiceChat = static_cast<FEOSVoiceChat&>(*StrongPtr);
			const bool bExecResult = EosVoiceChat.Exec(InWorld, Cmd, Ar);
			if (!bListRequested && bExecResult)
			{
				return true;
			}
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	return bListRequested;
}
#endif // UE_ALLOW_EXEC_COMMANDS

#endif // WITH_EOSVOICECHAT