// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVoiceChatModule.h"

#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"

#include "IEOSSDKManager.h"

#include COMPILED_PLATFORM_HEADER(EOSVoiceChat.h)

#include "EOSVoiceChatFactory.h"

IMPLEMENT_MODULE(FEOSVoiceChatModule, EOSVoiceChat);

void FEOSVoiceChatModule::StartupModule()
{
#if WITH_EOSVOICECHAT
	if (FParse::Param(FCommandLine::Get(), TEXT("NoEOS")))
	{
		return;
	}

	const FName EOSSharedModuleName = TEXT("EOSShared");
	const FName EOSSDKManagerFeatureName = TEXT("EOSSDKManager");

	if (!FModuleManager::Get().IsModuleLoaded(EOSSharedModuleName))
	{
		FModuleManager::Get().LoadModule(EOSSharedModuleName);
	}

	IEOSSDKManager* EOSSDKManager = nullptr;
	if (IModularFeatures::Get().IsModularFeatureAvailable(EOSSDKManagerFeatureName))
	{
		EOSSDKManager = &IModularFeatures::Get().GetModularFeature<IEOSSDKManager>(EOSSDKManagerFeatureName);
	}
	check(EOSSDKManager);

	EOSFactory = MakeShared<FEOSVoiceChatFactory, ESPMode::ThreadSafe>();
	IModularFeatures::Get().RegisterModularFeature(FEOSVoiceChatFactory::GetModularFeatureName(), EOSFactory.Get());

	EOSObj = EOSFactory->CreateInstance();
	IModularFeatures::Get().RegisterModularFeature(IVoiceChat::GetModularFeatureName(), EOSObj.Get());
#endif // WITH_EOSVOICECHAT
}

void FEOSVoiceChatModule::ShutdownModule()
{
#if WITH_EOSVOICECHAT
	if (EOSObj.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IVoiceChat::GetModularFeatureName(), EOSObj.Get());
		EOSObj->Uninitialize();
		EOSObj.Reset();
	}

	if (EOSFactory.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(FEOSVoiceChatFactory::GetModularFeatureName(), EOSFactory.Get());
		EOSFactory.Reset();
	}
#endif // WITH_EOSVOICECHAT
}