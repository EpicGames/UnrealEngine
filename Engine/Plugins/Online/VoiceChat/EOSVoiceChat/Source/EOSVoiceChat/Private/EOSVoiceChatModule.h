// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

using IVoiceChatPtr = TSharedPtr<class IVoiceChat, ESPMode::ThreadSafe>;
typedef TSharedPtr<class FEOSVoiceChatFactory, ESPMode::ThreadSafe> FEOSVoiceChatFactoryPtr;

class FEOSVoiceChatModule : public IModuleInterface
{
public:
	FEOSVoiceChatModule() = default;
	~FEOSVoiceChatModule() = default;

private:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

	/** Singleton EOS object */
	FEOSVoiceChatFactoryPtr EOSFactory;
	IVoiceChatPtr EOSObj;
};