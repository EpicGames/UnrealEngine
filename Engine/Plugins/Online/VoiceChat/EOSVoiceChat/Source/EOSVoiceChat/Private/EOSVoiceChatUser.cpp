// Copyright Epic Games, Inc. All Rights Reserved.

#include "EOSVoiceChatUser.h"

#if WITH_EOSVOICECHAT

#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h" 
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"

#include "EOSShared.h"
#include "EOSSharedTypes.h"
#include "EOSVoiceChat.h"
#include "EOSVoiceChatErrors.h"
#include "EOSVoiceChatModule.h"
#include "EOSVoiceChatTypes.h"
#include "VoiceChatErrors.h"
#include "EOSAudioDevicePool.h"
#include "IEOSSDKManager.h"

#include "eos_common.h"
#include "eos_lobby.h"
#include "eos_rtc.h"
#include "eos_rtc_audio.h"
#include "eos_sdk.h"

#define EOS_VOICE_TODO 0

namespace
{
	template<class TDelegate, class... TArgs>
	void TriggerCompletionDelegate(TDelegate& InOutDelegate, const TArgs&... Args)
	{
		TDelegate Delegate = InOutDelegate;
		InOutDelegate.Unbind();
		Delegate.ExecuteIfBound(Args...);
	}

	template <class TDelegate, class... TArgs>
	void TriggerCompletionDelegates(TArray<TDelegate>& InOutDelegates, const TArgs&... Args)
	{
		TArray<TDelegate> Delegates = MoveTemp(InOutDelegates);
		for (TDelegate& Delegate : Delegates)
		{
			Delegate.ExecuteIfBound(Args...);
		}
	}

	bool EOSNameContainsValidCharacters(const FString& Name)
	{
		static const FString AdditionalValidCharacters = TEXT("-_.");
		for (const TCHAR& Char : Name)
		{
			int32 Index;
			if (FChar::IsAlnum(Char) || AdditionalValidCharacters.FindChar(Char, Index))
			{
				continue;
			}

			return false;
		}

		return true;
	}

	FString LexToString(EOS_ERTCAudioStatus Status)
	{
		switch (Status)
		{
		case EOS_ERTCAudioStatus::EOS_RTCAS_Unsupported: return TEXT("EOS_RTCAS_Unsupported");
		case EOS_ERTCAudioStatus::EOS_RTCAS_Enabled: return TEXT("EOS_RTCAS_Enabled");
		case EOS_ERTCAudioStatus::EOS_RTCAS_Disabled: return TEXT("EOS_RTCAS_Disabled");
		case EOS_ERTCAudioStatus::EOS_RTCAS_AdminDisabled: return TEXT("EOS_RTCAS_AdminDisabled");
		case EOS_ERTCAudioStatus::EOS_RTCAS_NotListeningDisabled: return TEXT("EOS_RTCAS_NotListeningDisabled");
		default:
			checkNoEntry();
			return TEXT("Unknown");
		}
	}

	using FAudioBeforeSendCallback = TEOSGlobalCallback<EOS_RTCAudio_OnAudioBeforeSendCallback, EOS_RTCAudio_AudioBeforeSendCallbackInfo, FEOSVoiceChatUser>;
	using FUpdateSendingAudioCallback = TEOSCallback<EOS_RTCAudio_OnUpdateSendingCallback, EOS_RTCAudio_UpdateSendingCallbackInfo, FEOSVoiceChatUser>;
}

static TAutoConsoleVariable<bool> CVarFakeAudioInputEnabled(
	TEXT("EOSVoiceChat.FakeAudioInputEnabled"),
	false,
	TEXT("Whether Fake Audio input is enabled or not."));

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<bool> CVarChannelEchoEnabled(
	TEXT("EOSVoiceChat.ChannelEchoEnabled"),
	false,
	TEXT("Whether channel echo should be enabled or not for testing with a single participant in the room (default false)."));
#endif

FEOSVoiceChatUser::FEOSVoiceChatUser(FEOSVoiceChat& InEOSVoiceChat)
	: EOSVoiceChat(InEOSVoiceChat)
{
}

FEOSVoiceChatUser::~FEOSVoiceChatUser()
{
	bInDestructor = true;
	ClearLoginSession();
}

#pragma region IVoiceChatUser
void FEOSVoiceChatUser::SetSetting(const FString& Name, const FString& Value)
{
	EOSVOICECHATUSER_LOG(Verbose, TEXT("SetSetting - %s - Parameter: %s"), *Name, *Value);

	if (Name.Equals(TEXT("HardwareAEC"), ESearchCase::IgnoreCase))
	{
		SetHardwareAECEnabled(Value.ToBool());
	}
	else if (Name.Equals(TEXT("NotListening")))
	{
		FString ChannelName;
		FString Enabled;
		if (Value.Split(":", &ChannelName, &Enabled))
		{
			FChannelSession& ChannelSession = GetChannelSession(ChannelName);
			bool bIsNotListening = FCString::ToBool(*Enabled);
			if (bIsNotListening != ChannelSession.bIsNotListening)
			{
				ChannelSession.bIsNotListening = bIsNotListening;
				ApplySendingOptions(ChannelSession);
				ApplyReceivingOptions(ChannelSession);
				for (TPair<FString, FChannelParticipant>& ChannelParticipantPair : ChannelSession.Participants)
				{
					const FString& ChannelParticipantName = ChannelParticipantPair.Key;
					FChannelParticipant& ChannelParticipant = ChannelParticipantPair.Value;
					if (!ChannelSession.IsLocalUser(ChannelParticipant))
					{
						ApplyPlayerReceivingOptions(GetGlobalParticipant(ChannelParticipantName), ChannelSession, ChannelParticipant);
					}
				}
			}
		}
	}
}

FString FEOSVoiceChatUser::GetSetting(const FString& Name)
{
	if (Name.Equals(TEXT("HardwareAEC"), ESearchCase::IgnoreCase))
	{
		return LexToString(AudioInputOptions.bPlatformAEC);
	}

	return FString();
}

void FEOSVoiceChatUser::SetAudioInputVolume(float InVolume)
{
	EOSVOICECHATUSER_LOG(Verbose, TEXT("SetAudioInputVolume %f"), InVolume);

	InVolume = FMath::Clamp(InVolume, 0.0f, 2.0f);

	if (AudioInputOptions.Volume != InVolume)
	{
		AudioInputOptions.Volume = InVolume;

		ApplyAudioInputOptions();
	}
}

void FEOSVoiceChatUser::SetAudioOutputVolume(float InVolume)
{
	EOSVOICECHATUSER_LOG(Verbose, TEXT("SetAudioOutputVolume %f"), InVolume);

	InVolume = FMath::Clamp(InVolume, 0.0f, 2.0f);

	if (AudioOutputOptions.Volume != InVolume)
	{
		AudioOutputOptions.Volume = InVolume;

		ApplyAudioOutputOptions();
	}
}

float FEOSVoiceChatUser::GetAudioInputVolume() const
{
	return AudioInputOptions.Volume;
}

float FEOSVoiceChatUser::GetAudioOutputVolume() const
{
	return AudioOutputOptions.Volume;
}

void FEOSVoiceChatUser::SetAudioInputDeviceMuted(bool bIsMuted)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("SetAudioInputDeviceMuted %s"), *LexToString(bIsMuted));

	if (AudioInputOptions.bMuted != bIsMuted)
	{
		AudioInputOptions.bMuted = bIsMuted;

		ApplySendingOptions();
	}
}

void FEOSVoiceChatUser::SetAudioOutputDeviceMuted(bool bIsMuted)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("SetAudioOutputDeviceMuted %s"), *LexToString(bIsMuted));

	if (AudioOutputOptions.bMuted != bIsMuted)
	{
		AudioOutputOptions.bMuted = bIsMuted;

		ApplyAudioOutputOptions();
	}
}

bool FEOSVoiceChatUser::GetAudioInputDeviceMuted() const
{
	return AudioInputOptions.bMuted;
}

bool FEOSVoiceChatUser::GetAudioOutputDeviceMuted() const
{
	return AudioOutputOptions.bMuted;
}

TArray<FVoiceChatDeviceInfo> FEOSVoiceChatUser::GetAvailableInputDeviceInfos() const
{
	return EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedInputDeviceInfos();
}

TArray<FVoiceChatDeviceInfo> FEOSVoiceChatUser::GetAvailableOutputDeviceInfos() const
{
	return EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedOutputDeviceInfos();
}

void FEOSVoiceChatUser::SetInputDeviceId(const FString& InputDeviceId)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("SetInputDeviceId [%s]"), *InputDeviceId);
	
	AudioInputOptions.SpecificDeviceInfo.Reset();

	if (!InputDeviceId.IsEmpty())
	{
		for (const FVoiceChatDeviceInfo& AvailableInputDevice : GetAvailableInputDeviceInfos())
		{
			if (InputDeviceId == AvailableInputDevice.Id)
			{
				AudioInputOptions.SpecificDeviceInfo.Emplace(AvailableInputDevice);
				break;
			}
		}
	}

	const FVoiceChatDeviceInfo& EffectiveDevice = GetInputDeviceInfo();
	EOSVOICECHATUSER_LOG(Log, TEXT("SetInputDeviceId effective device Id=[%s] DisplayName=[%s]"), *EffectiveDevice.Id, *EffectiveDevice.DisplayName);

	ApplyAudioInputOptions();
}

void FEOSVoiceChatUser::SetOutputDeviceId(const FString& OutputDeviceId)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("SetOutputDeviceId [%s]"), *OutputDeviceId);

	AudioOutputOptions.SpecificDeviceInfo.Reset();

	if (!OutputDeviceId.IsEmpty())
	{
		for (const FVoiceChatDeviceInfo& AvailableOutputDevice : GetAvailableOutputDeviceInfos())
		{
			if (OutputDeviceId == AvailableOutputDevice.Id)
			{
				AudioOutputOptions.SpecificDeviceInfo.Emplace(AvailableOutputDevice);
				break;
			}
		}
	}

	const FVoiceChatDeviceInfo& EffectiveDevice = GetOutputDeviceInfo();
	EOSVOICECHATUSER_LOG(Log, TEXT("SetOutputDeviceId effective device Id=[%s] DisplayName=[%s]"), *EffectiveDevice.Id, *EffectiveDevice.DisplayName);

	ApplyAudioOutputOptions();
}

FVoiceChatDeviceInfo FEOSVoiceChatUser::GetInputDeviceInfo() const
{
	if (AudioInputOptions.SpecificDeviceInfo.IsSet())
	{
		return AudioInputOptions.SpecificDeviceInfo.GetValue();
	}
	else
	{
		return GetDefaultInputDeviceInfo();
	}
}

FVoiceChatDeviceInfo FEOSVoiceChatUser::GetOutputDeviceInfo() const
{
	if (AudioOutputOptions.SpecificDeviceInfo.IsSet())
	{
		return AudioOutputOptions.SpecificDeviceInfo.GetValue();
	}
	else
	{
		return GetDefaultOutputDeviceInfo();
	}
}

FVoiceChatDeviceInfo FEOSVoiceChatUser::GetDefaultInputDeviceInfo() const
{
	FVoiceChatDeviceInfo DefaultDeviceInfo;

	if (EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedInputDeviceInfos().IsValidIndex(EOSVoiceChat.InitSession.EosAudioDevicePool->GetDefaultInputDeviceInfoIdx()))
	{
		DefaultDeviceInfo = EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedInputDeviceInfos()[EOSVoiceChat.InitSession.EosAudioDevicePool->GetDefaultInputDeviceInfoIdx()];
	}

	return DefaultDeviceInfo;
}

FVoiceChatDeviceInfo FEOSVoiceChatUser::GetDefaultOutputDeviceInfo() const
{
	FVoiceChatDeviceInfo DefaultDeviceInfo;

	if (EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedOutputDeviceInfos().IsValidIndex(EOSVoiceChat.InitSession.EosAudioDevicePool->GetDefaultOutputDeviceInfoIdx()))
	{
		DefaultDeviceInfo = EOSVoiceChat.InitSession.EosAudioDevicePool->GetCachedOutputDeviceInfos()[EOSVoiceChat.InitSession.EosAudioDevicePool->GetDefaultOutputDeviceInfoIdx()];
	}

	return DefaultDeviceInfo;
}

void FEOSVoiceChatUser::Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& LoginCompleteDelegate)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();

	if (!IsInitialized())
	{
		Result = VoiceChat::Errors::NotInitialized();
	}
	else if (!IsConnected())
	{
		Result = VoiceChat::Errors::NotConnected();
	}
	else if (IsLoggedIn())
	{
		if (PlayerName == GetLoggedInPlayerName())
		{
			LoginCompleteDelegate.ExecuteIfBound(PlayerName, FVoiceChatResult::CreateSuccess());
			return;
		}
		else
		{
			Result = VoiceChat::Errors::OtherUserLoggedIn();
		}
	}
	else if (LoginSession.State == ELoginState::LoggingIn)
	{
		Result = VoiceChat::Errors::AlreadyInProgress();
	}
	else if (PlayerName.IsEmpty())
	{
		Result = VoiceChat::Errors::InvalidArgument(TEXT("PlayerName empty"));
	}
	else if (!EOSNameContainsValidCharacters(PlayerName))
	{
		Result = VoiceChat::Errors::InvalidArgument(TEXT("PlayerName invalid characters"));
	}

	if (!Result.IsSuccess())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("Login PlayerName:%s %s"), *PlayerName, *LexToString(Result));
		LoginCompleteDelegate.ExecuteIfBound(PlayerName, Result);
		return;
	}

	const EOS_ProductUserId LocalUserProductUserId = EOSProductUserIdFromString(*PlayerName);
	if (EOS_ProductUserId_IsValid(LocalUserProductUserId))
	{
		LoginSession.PlatformId = PlatformId;
		LoginSession.PlayerName = PlayerName;
		LoginSession.LocalUserProductUserId = LocalUserProductUserId;
		LoginSession.State = ELoginState::LoggedIn;

		BindLoginCallbacks();

		LoginCompleteDelegate.ExecuteIfBound(LoginSession.PlayerName, FVoiceChatResult::CreateSuccess());
		OnVoiceChatLoggedInDelegate.Broadcast(LoginSession.PlayerName);

		ApplyAudioInputOptions();
		ApplyAudioOutputOptions();
	}
	else
	{
		Result = VoiceChat::Errors::InvalidArgument(TEXT("PlayerName->ProductUserId conversion failed"));
		EOSVOICECHATUSER_LOG(Warning, TEXT("Login PlayerName:%s %s"), *PlayerName, *LexToString(Result));

		LoginCompleteDelegate.ExecuteIfBound(PlayerName, Result);
	}
}

void FEOSVoiceChatUser::Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();

	if (!IsInitialized())
	{
		Result = VoiceChat::Errors::NotInitialized();
	}
	else if (!IsConnected())
	{
		Result = VoiceChat::Errors::NotConnected();
	}
	else if (LoginSession.State != ELoginState::LoggedIn &&
		LoginSession.State != ELoginState::LoggingOut)
	{
		Result = VoiceChat::Errors::NotLoggedIn();
	}

	if (!Result.IsSuccess())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("Logout %s"), *LexToString(Result));
		Delegate.ExecuteIfBound(FString(), Result);
		return;
	}

	LogoutInternal(Delegate);
}

bool FEOSVoiceChatUser::IsLoggingIn() const
{
	return LoginSession.State == ELoginState::LoggingIn;
}

bool FEOSVoiceChatUser::IsLoggedIn() const
{
	return LoginSession.State == ELoginState::LoggedIn;
}

FString FEOSVoiceChatUser::GetLoggedInPlayerName() const
{
	return IsLoggedIn() ? LoginSession.PlayerName : FString();
}

void FEOSVoiceChatUser::BlockPlayers(const TArray<FString>& PlayerNames)
{
	if (IsLoggedIn())
	{
		EOSVOICECHATUSER_LOG(Log, TEXT("BlockPlayers PlayerNames=[%s]"), *FString::Join(PlayerNames, TEXT(", ")));

		for (const FString& PlayerName : PlayerNames)
		{
			FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
			GlobalParticipant.bBlocked = true;

			for (TPair<FString, FChannelSession>& ChannelSessionPair : LoginSession.ChannelSessions)
			{
				FChannelSession& ChannelSession = ChannelSessionPair.Value;
				if (ChannelSession.JoinState == EChannelJoinState::Joined)
				{
					if(FChannelParticipant* ChannelParticipant = ChannelSession.Participants.Find(PlayerName))
					{
						ApplyPlayerBlock(GlobalParticipant, ChannelSession, *ChannelParticipant);
					}
				}
			}
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("BlockPlayers called while logged out"));
	}
}

void FEOSVoiceChatUser::UnblockPlayers(const TArray<FString>& PlayerNames)
{
	if (IsLoggedIn())
	{
		EOSVOICECHATUSER_LOG(Log, TEXT("UnblockPlayers PlayerNames=[%s]"), *FString::Join(PlayerNames, TEXT(", ")));

		for (const FString& PlayerName : PlayerNames)
		{
			FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
			GlobalParticipant.bBlocked = false;

			for (TPair<FString, FChannelSession>& ChannelSessionPair : LoginSession.ChannelSessions)
			{
				FChannelSession& ChannelSession = ChannelSessionPair.Value;
				if (ChannelSession.JoinState == EChannelJoinState::Joined)
				{
					if (FChannelParticipant* ChannelParticipant = ChannelSession.Participants.Find(PlayerName))
					{
						ApplyPlayerBlock(GlobalParticipant, ChannelSession, *ChannelParticipant);
					}
				}
			}
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("UnblockPlayers called while logged out"));
	}
}

void FEOSVoiceChatUser::JoinChannel(const FString& ChannelName, const FString& ChannelCredentialsStr, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
	FEOSVoiceChatChannelCredentials ChannelCredentials;

	if (!IsInitialized())
	{
		Result = VoiceChat::Errors::NotInitialized();
	}
	else if (!IsConnected())
	{
		Result = VoiceChat::Errors::NotConnected();
	}
	else if (!IsLoggedIn())
	{
		Result = VoiceChat::Errors::NotLoggedIn();
	}
	else if (ChannelName.IsEmpty())
	{
		Result = VoiceChat::Errors::InvalidArgument(TEXT("ChannelName empty"));
	}
	else if (!EOSNameContainsValidCharacters(ChannelName))
	{
		Result = VoiceChat::Errors::InvalidArgument(TEXT("ChannelName invalid characters"));
	}
	else if (!ChannelCredentials.FromJson(ChannelCredentialsStr))
	{
		Result = VoiceChat::Errors::CredentialsInvalid(TEXT("Failed to deserialize ChannelCredentials"));
	}

	if (Result.IsSuccess())
	{
		FChannelSession& ChannelSession = GetChannelSession(ChannelName);
		if (ChannelSession.JoinState == EChannelJoinState::Joined)
		{
			Delegate.ExecuteIfBound(ChannelName, FVoiceChatResult::CreateSuccess());
			return;
		}
		else if (ChannelSession.JoinState == EChannelJoinState::Joining)
		{
			Result = VoiceChat::Errors::ChannelJoinInProgress();
		}
		else if (ChannelSession.JoinState == EChannelJoinState::Leaving)
		{
			Result = VoiceChat::Errors::ChannelLeaveInProgress();
		}
		else
		{
			if (!ChannelCredentials.OverrideUserId.IsEmpty())
			{
				ChannelSession.PlayerName = ChannelCredentials.OverrideUserId;
			}
			else
			{
				ChannelSession.PlayerName = LoginSession.PlayerName;
			}
			ChannelSession.ChannelName = ChannelName;
			ChannelSession.ChannelType = ChannelType;
			ChannelSession.JoinState = EChannelJoinState::Joining;
			ChannelSession.JoinDelegate = Delegate;

			bFakeAudioInput = CVarFakeAudioInputEnabled.GetValueOnAnyThread();
			ApplyAudioInputOptions(); // Reapply audio options based on CVar properties

			BindChannelCallbacks(ChannelSession);
			ApplySendingOptions(ChannelSession);
			ApplyReceivingOptions(ChannelSession);

			const FTCHARToUTF8 Utf8RoomName(*ChannelName);
			const FTCHARToUTF8 Utf8ClientBaseUrl(*ChannelCredentials.ClientBaseUrl);
			const FTCHARToUTF8 Utf8ParticipantToken(*ChannelCredentials.ParticipantToken);
			const EOS_ProductUserId ParticipantId = EOSProductUserIdFromString(*ChannelSession.PlayerName);

			// Attempt to join the channel
			{
				EOS_RTC_JoinRoomOptions JoinOptions = {};
				JoinOptions.ApiVersion = 1;
				UE_EOS_CHECK_API_MISMATCH(EOS_RTC_JOINROOM_API_LATEST, 1);
				JoinOptions.LocalUserId = LoginSession.LocalUserProductUserId;
				JoinOptions.RoomName = Utf8RoomName.Get();
				JoinOptions.ClientBaseUrl = Utf8ClientBaseUrl.Get();
				JoinOptions.ParticipantId = ParticipantId;
				JoinOptions.ParticipantToken = Utf8ParticipantToken.Get();
#if !UE_BUILD_SHIPPING
				const bool bEnableEcho = ChannelSession.ChannelType == EVoiceChatChannelType::Echo || CVarChannelEchoEnabled.GetValueOnAnyThread();
				JoinOptions.Flags |= bEnableEcho ? EOS_RTC_JOINROOMFLAGS_ENABLE_ECHO : 0x0;
#endif
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
				QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_JoinRoom);
				EOS_RTC_JoinRoom(GetRtcInterface(), &JoinOptions, this, &FEOSVoiceChatUser::OnJoinRoomStatic);
			}
		}
	}

	if (!Result.IsSuccess())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("JoinChannel ChannelName:%s %s"), *ChannelName, *LexToString(Result));
		Delegate.ExecuteIfBound(ChannelName, Result);
		return;
	}
}

void FEOSVoiceChatUser::LeaveChannel(const FString& ChannelName, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate)
{
	FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();

	if (!IsInitialized())
	{
		Result = VoiceChat::Errors::NotInitialized();
	}
	else if (!IsConnected())
	{
		Result = VoiceChat::Errors::NotConnected();
	}
	else if (!IsLoggedIn())
	{
		Result = VoiceChat::Errors::NotLoggedIn();
	}
	else if (ChannelName.IsEmpty())
	{
		Result = VoiceChat::Errors::InvalidArgument();
	}
	else
	{
		FChannelSession& ChannelSession = GetChannelSession(ChannelName);
		if (ChannelSession.JoinState != EChannelJoinState::Joined)
		{
			Result = VoiceChat::Errors::NotInChannel();
		}
		else if (ChannelSession.IsLobbySession())
		{
			EOSVOICECHATUSER_LOG(Error, TEXT("LeaveChannel ChannelName: lobby rooms can only be removed with RemoveLobbyRoom."));
			Result = VoiceChat::Errors::NotPermitted();
		}
	}

	if (!Result.IsSuccess())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("LeaveChannel ChannelName:%s %s"), *ChannelName, *LexToString(Result));
		Delegate.ExecuteIfBound(ChannelName, Result);
		return;
	}

	LeaveChannelInternal(ChannelName, Delegate);
}

void FEOSVoiceChatUser::Set3DPosition(const FString& ChannelName, const FVector& Position)
{
	// Unimplemented
}

TArray<FString> FEOSVoiceChatUser::GetChannels() const
{
	TArray<FString> ChannelNames;
	for (const TPair<FString, FChannelSession>& ChannelSessionPair : LoginSession.ChannelSessions)
	{
		const FString& ChannelName = ChannelSessionPair.Key;
		const FChannelSession& ChannelSession = ChannelSessionPair.Value;
		if (ChannelSession.JoinState == EChannelJoinState::Joined)
		{
			ChannelNames.Add(ChannelName);
		}
	}
	return ChannelNames;
}

TArray<FString> FEOSVoiceChatUser::GetPlayersInChannel(const FString& ChannelName) const
{
	TArray<FString> PlayerNames;
	GetChannelSession(ChannelName).Participants.GenerateKeyArray(PlayerNames);
	return PlayerNames;
}

EVoiceChatChannelType FEOSVoiceChatUser::GetChannelType(const FString& ChannelName) const
{
	return GetChannelSession(ChannelName).ChannelType;
}

bool FEOSVoiceChatUser::IsPlayerTalking(const FString& PlayerName) const
{
	return GetGlobalParticipant(PlayerName).bTalking;
}

void FEOSVoiceChatUser::SetPlayerMuted(const FString& PlayerName, bool bAudioMuted)
{
	if (!IsLoggedIn())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("SetPlayerMuted called while logged out"));
		return;
	}

	FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
	const bool bAudioMutedChanged = GlobalParticipant.bAudioMuted != bAudioMuted;

	EOSVOICECHATUSER_CLOG(bAudioMutedChanged, Log, TEXT("SetPlayerMuted PlayerName=[%s] bAudioMuted=[%s]"), *PlayerName, *LexToString(bAudioMuted));
	EOSVOICECHATUSER_CLOG(!bAudioMutedChanged, Verbose, TEXT("SetPlayerMuted PlayerName=[%s] bAudioMuted=[%s] (No change)"), *PlayerName, *LexToString(bAudioMuted));

	if (bAudioMutedChanged)
	{
		GlobalParticipant.bAudioMuted = bAudioMuted;

		ApplyPlayerReceivingOptions(PlayerName);
	}
}

bool FEOSVoiceChatUser::IsPlayerMuted(const FString& PlayerName) const
{
	if (!IsLoggedIn())
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("IsPlayerMuted called while logged out"));
		return false;
	}

	return GetGlobalParticipant(PlayerName).bAudioMuted;
}

void FEOSVoiceChatUser::SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bAudioMuted)
{
	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		if (FChannelParticipant* ChannelParticipant = ChannelSession->Participants.Find(PlayerName))
		{
			const bool bMutedInChannelChanged = ChannelParticipant->bMutedInChannel != bAudioMuted;

			EOSVOICECHATUSER_CLOG(bMutedInChannelChanged, Log, TEXT("SetChannelPlayerMuted PlayerName=[%s] bMutedInChannel=[%s]"), *PlayerName, *LexToString(bAudioMuted));
			EOSVOICECHATUSER_CLOG(!bMutedInChannelChanged, Verbose, TEXT("SetChannelPlayerMuted PlayerName=[%s] bMutedInChannel=[%s] (No change)"), *PlayerName, *LexToString(bAudioMuted));

			if (bMutedInChannelChanged)
			{
				ChannelParticipant->bMutedInChannel = bAudioMuted;

				FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
				ApplyPlayerReceivingOptions(GlobalParticipant, *ChannelSession, *ChannelParticipant);
			}
		}
	}
}

bool FEOSVoiceChatUser::IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const
{
	if (const FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		if (const FChannelParticipant* ChannelParticipant = ChannelSession->Participants.Find(PlayerName))
		{
			return ChannelParticipant->bMutedInChannel;
		}
	}

	return false;
}

void FEOSVoiceChatUser::SetPlayerVolume(const FString& PlayerName, float Volume)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("SetPlayerVolume PlayerName=[%s] Volume=[%s]"), *PlayerName, *LexToString(Volume));

	FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
	GlobalParticipant.Volume = FMath::Clamp(Volume, 0.0f, 2.0f);

	ApplyPlayerReceivingOptions(PlayerName);
}

float FEOSVoiceChatUser::GetPlayerVolume(const FString& PlayerName) const
{
	return GetGlobalParticipant(PlayerName).Volume;
}

void FEOSVoiceChatUser::TransmitToAllChannels()
{
	if (TransmitState.Mode != EVoiceChatTransmitMode::All)
	{
		TransmitState.Mode = EVoiceChatTransmitMode::All;
		ApplySendingOptions();
	}
}

void FEOSVoiceChatUser::TransmitToNoChannels()
{
	if (TransmitState.Mode != EVoiceChatTransmitMode::None)
	{
		TransmitState.Mode = EVoiceChatTransmitMode::None;
		ApplySendingOptions();
	}
}

void FEOSVoiceChatUser::TransmitToSpecificChannels(const TSet<FString>& ChannelNames)
{
	if (TransmitState.Mode != EVoiceChatTransmitMode::SpecificChannels || 
		!TransmitState.SpecificChannels.Difference(ChannelNames).IsEmpty() || 
		!ChannelNames.Difference(TransmitState.SpecificChannels).IsEmpty())
	{
		TransmitState.Mode = EVoiceChatTransmitMode::SpecificChannels;
		TransmitState.SpecificChannels = ChannelNames;
		ApplySendingOptions();
	}
}

EVoiceChatTransmitMode FEOSVoiceChatUser::GetTransmitMode() const
{
	return TransmitState.Mode;
}

TSet<FString> FEOSVoiceChatUser::GetTransmitChannels() const
{
	if (TransmitState.Mode == EVoiceChatTransmitMode::SpecificChannels)
	{
		return TransmitState.SpecificChannels;
	}
	return TSet<FString>();
}

FDelegateHandle FEOSVoiceChatUser::StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate)
{
#if EOS_VOICE_TODO
	// TODO no API for just recording from the mic
	FScopeLock Lock(&AudioRecordLock);
	return OnVoiceChatRecordSamplesAvailableDelegate.Add(Delegate);
#else
	return FDelegateHandle();
#endif
}

void FEOSVoiceChatUser::StopRecording(FDelegateHandle Handle)
{
#if EOS_VOICE_TODO
	// TODO no API for just recording from the mic.
	FScopeLock Lock(&AudioRecordLock);
	OnVoiceChatRecordSamplesAvailableDelegate.Remove(Handle);
#endif
}

FDelegateHandle FEOSVoiceChatUser::RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate)
{
	FScopeLock Lock(&BeforeCaptureAudioSentLock);

	return OnVoiceChatAfterCaptureAudioReadDelegate.Add(Delegate);
}

void FEOSVoiceChatUser::UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle)
{
	FScopeLock Lock(&BeforeCaptureAudioSentLock);

	OnVoiceChatAfterCaptureAudioReadDelegate.Remove(Handle);
}

FDelegateHandle FEOSVoiceChatUser::RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate)
{
	FScopeLock Lock(&BeforeCaptureAudioSentLock);

	return OnVoiceChatBeforeCaptureAudioSentDelegate.Add(Delegate);
}

void FEOSVoiceChatUser::UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle)
{
	FScopeLock Lock(&BeforeCaptureAudioSentLock);

	OnVoiceChatBeforeCaptureAudioSentDelegate.Remove(Handle);
}

FDelegateHandle FEOSVoiceChatUser::RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate)
{
	FScopeLock Lock(&BeforeRecvMixedAudioRenderedLock);
	return OnVoiceChatBeforeRecvMixedAudioRenderedDelegate.Add(Delegate);
}

void FEOSVoiceChatUser::UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle)
{
	FScopeLock Lock(&BeforeRecvMixedAudioRenderedLock);
	OnVoiceChatBeforeRecvMixedAudioRenderedDelegate.Remove(Handle);
}

FDelegateHandle FEOSVoiceChatUser::RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate)
{
	FScopeLock Lock(&BeforeRecvUnmixedAudioRenderedLock);
	return OnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate.Add(Delegate);
}

void FEOSVoiceChatUser::UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle)
{
	FScopeLock Lock(&BeforeRecvUnmixedAudioRenderedLock);
	OnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate.Remove(Handle);
}

FString FEOSVoiceChatUser::InsecureGetLoginToken(const FString& PlayerName)
{
	return FString();
}

FString FEOSVoiceChatUser::InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties)
{
	FEOSVoiceChatChannelCredentials ChannelCredentials;
	GConfig->GetString(TEXT("EOSVoiceChat"), TEXT("InsecureClientBaseUrl"), ChannelCredentials.ClientBaseUrl, GEngineIni);

	ChannelCredentials.ParticipantToken = TEXT("0:EOSVoiceChatTest:`RoomName:`UserName:sss");
	ChannelCredentials.ParticipantToken.ReplaceInline(TEXT("`UserName"), *GetLoggedInPlayerName());
	ChannelCredentials.ParticipantToken.ReplaceInline(TEXT("`RoomName"), *ChannelName);

	return ChannelCredentials.ToJson(false);
}
#pragma endregion IVoiceChatUser

bool FEOSVoiceChatUser::AddLobbyRoom(const FString& LobbyId)
{
	if (!GetLobbyInterface())
	{
		EOSVOICECHATUSER_LOG(Error, TEXT("AddLobbyRoom Lobby interface invalid."));
		return false;
	}

	const FTCHARToUTF8 Utf8LobbyId(*LobbyId);

	EOS_Lobby_GetRTCRoomNameOptions GetRoomNameOptions = {};
	GetRoomNameOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_GETRTCROOMNAME_API_LATEST, 1);
	GetRoomNameOptions.LobbyId = Utf8LobbyId.Get();
	GetRoomNameOptions.LocalUserId = LoginSession.LocalUserProductUserId;
	char Utf8RoomName[256];
	uint32_t Utf8RoomNameLength = UE_ARRAY_COUNT(Utf8RoomName);
	EOS_EResult EosResult = EOS_Lobby_GetRTCRoomName(GetLobbyInterface(), &GetRoomNameOptions, Utf8RoomName, &Utf8RoomNameLength);
	if (EosResult == EOS_EResult::EOS_Success)
	{
		const FString ChannelName = UTF8_TO_TCHAR(Utf8RoomName);
		FChannelSession& ChannelSession = GetChannelSession(ChannelName);
		ChannelSession.LobbyId = LobbyId;
		ChannelSession.JoinState = EChannelJoinState::Joined;
		ChannelSession.PlayerName = LoginSession.PlayerName;

		LoginSession.LobbyIdToChannelName.Emplace(ChannelSession.LobbyId, ChannelSession.ChannelName);

		bFakeAudioInput = CVarFakeAudioInputEnabled.GetValueOnAnyThread();
		ApplyAudioInputOptions(); // Reapply audio options based on CVar properties

		BindChannelCallbacks(ChannelSession);
		ApplySendingOptions(ChannelSession);

		// Call UpdateReceiving once to set the default receiving state for all participants
		{
			EOS_RTCAudio_UpdateReceivingOptions UpdateReceivingOptions = {};
			UpdateReceivingOptions.ApiVersion = 1;
			UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UPDATERECEIVING_API_LATEST, 1);
			UpdateReceivingOptions.LocalUserId = LoginSession.LocalUserProductUserId;
			UpdateReceivingOptions.RoomName = Utf8RoomName;
			UpdateReceivingOptions.ParticipantId = nullptr;
			UpdateReceivingOptions.bAudioEnabled = ChannelSession.bIsNotListening ? EOS_FALSE : EOS_TRUE;

			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
			QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_UpdateReceiving);
			EOS_RTCAudio_UpdateReceiving(EOS_RTC_GetAudioInterface(GetRtcInterface()), &UpdateReceivingOptions, this, &FEOSVoiceChatUser::OnUpdateReceivingAudioStatic);
		}
		
		// Get the current lobby connectedness.
		EOS_Lobby_IsRTCRoomConnectedOptions IsRoomConnectedOptions = {};
		IsRoomConnectedOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ISRTCROOMCONNECTED_API_LATEST, 1);
		IsRoomConnectedOptions.LobbyId = Utf8LobbyId.Get();
		IsRoomConnectedOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		EOS_Bool bLobbyChannelConnected = false;
		EosResult = EOS_Lobby_IsRTCRoomConnected(GetLobbyInterface(), &IsRoomConnectedOptions, &bLobbyChannelConnected);
		if (EosResult == EOS_EResult::EOS_Success)
		{
			ChannelSession.bLobbyChannelConnected = bLobbyChannelConnected == EOS_TRUE;
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("EOS_Lobby_IsLobbyRTCRoomConnected failed error=%s"), *LexToString(EosResult));
		}

		OnVoiceChatChannelJoinedDelegate.Broadcast(ChannelSession.ChannelName);

		FChannelParticipant& ChannelParticipant = ChannelSession.Participants.Add(ChannelSession.PlayerName);
		ChannelParticipant.PlayerName = ChannelSession.PlayerName;
		OnVoiceChatPlayerAddedDelegate.Broadcast(ChannelSession.ChannelName, ChannelSession.PlayerName);

		// TODO Participant query

		return true;
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("EOS_Lobby_GetLobbyRTCRoomName failed error=%s"), *LexToString(EosResult));
	}

	return false;
}

bool FEOSVoiceChatUser::RemoveLobbyRoom(const FString& LobbyId)
{
	if (!GetLobbyInterface())
	{
		EOSVOICECHATUSER_LOG(Error, TEXT("RemoveLobbyRoom Lobby interface invalid."));
		return false;
	}

	if (const FString* ChannelName = LoginSession.LobbyIdToChannelName.Find(LobbyId))
	{
		FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(*ChannelName);
		if (ensure(ChannelSession))
		{
			ChannelSession->JoinState = EChannelJoinState::NotJoined;

			OnVoiceChatChannelExitedDelegate.Broadcast(ChannelSession->ChannelName, FVoiceChatResult::CreateSuccess());

			LoginSession.LobbyIdToChannelName.Remove(ChannelSession->LobbyId);
			RemoveChannelSession(ChannelSession->ChannelName);
			return true;
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("RemoveLobbyRoom LobbyId=[%s] not found"), *LobbyId);
	}

	return false;
}

void FEOSVoiceChatUser::RtcRegisterUser(const FString& UserId, const FOnVoiceChatUserRtcRegisterUserCompleteDelegate& Delegate)
{
	const FTCHARToUTF8 Utf8UserId(*UserId);

	EOS_RTCAudio_RegisterPlatformAudioUserOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_REGISTERPLATFORMAUDIOUSER_API_LATEST, 1);
	Options.UserId = Utf8UserId.Get();

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_RegisterAudioUser);
	const EOS_EResult EosResult = EOS_RTCAudio_RegisterPlatformAudioUser(EOS_RTC_GetAudioInterface(GetRtcInterface()), &Options);
	if (EosResult != EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("RtcRegisterUser failed: %s"), *LexToString(EosResult));
	}

	Delegate.ExecuteIfBound(EosResult);
}

void FEOSVoiceChatUser::RtcUnregisterUser(const FString& UserId, const FOnVoiceChatUserRtcUnregisterUserCompleteDelegate& Delegate)
{
	const FTCHARToUTF8 Utf8UserId(*UserId);

	EOS_RTCAudio_UnregisterPlatformAudioUserOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UNREGISTERPLATFORMAUDIOUSER_API_LATEST, 1);
	Options.UserId = Utf8UserId.Get();

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_UnregisterAudioUser);
	const EOS_EResult EosResult = EOS_RTCAudio_UnregisterPlatformAudioUser(EOS_RTC_GetAudioInterface(GetRtcInterface()), &Options);
	if (EosResult != EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("RtcUnregisterUser failed: %s"), *LexToString(EosResult));
	}

	Delegate.ExecuteIfBound(EosResult);
}

void FEOSVoiceChatUser::SetHardwareAECEnabled(bool bEnabled)
{
	AudioInputOptions.bPlatformAEC = bEnabled;
	ApplyAudioInputOptions();
}

bool FEOSVoiceChatUser::IsInitialized()
{
	return EOSVoiceChat.IsInitialized();
}

bool FEOSVoiceChatUser::IsConnected()
{
	return EOSVoiceChat.IsConnected();
}

FEOSVoiceChatUser::FGlobalParticipant& FEOSVoiceChatUser::GetGlobalParticipant(const FString& PlayerName)
{
	if (FGlobalParticipant* Participant = LoginSession.Participants.Find(PlayerName))
	{
		return *Participant;
	}

	FGlobalParticipant& NewParticipant = LoginSession.Participants.Add(PlayerName);
	NewParticipant.PlayerName = PlayerName;
	return NewParticipant;
}

const FEOSVoiceChatUser::FGlobalParticipant& FEOSVoiceChatUser::GetGlobalParticipant(const FString& PlayerName) const
{
	if (const FGlobalParticipant* Participant = LoginSession.Participants.Find(PlayerName))
	{
		return *Participant;
	}
	else
	{
		static FGlobalParticipant NullParticipant;
		return NullParticipant;
	}
}

FEOSVoiceChatUser::FChannelSession& FEOSVoiceChatUser::GetChannelSession(const FString& ChannelName)
{
	if (FChannelSession* Session = LoginSession.ChannelSessions.Find(ChannelName))
	{
		return *Session;
	}

	FChannelSession& NewSession = LoginSession.ChannelSessions.Add(ChannelName);
	NewSession.ChannelName = ChannelName;
	NewSession.ChannelType = EVoiceChatChannelType::NonPositional;
	return NewSession;
}

const FEOSVoiceChatUser::FChannelSession& FEOSVoiceChatUser::GetChannelSession(const FString& ChannelName) const
{
	if (const FChannelSession* Session = LoginSession.ChannelSessions.Find(ChannelName))
	{
		return *Session;
	}
	else
	{
		static FChannelSession NullSession;
		return NullSession;
	}
}

void FEOSVoiceChatUser::RemoveChannelSession(const FString& ChannelName)
{
	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		UnbindChannelCallbacks(*ChannelSession);
		LoginSession.ChannelSessions.Remove(ChannelName);
	}
}

void FEOSVoiceChatUser::ApplyAudioInputOptions()
{
	if (IsLoggedIn())
	{
		const TCHAR* DeviceIdPtr = AudioInputOptions.SpecificDeviceInfo.IsSet() ?
			*AudioInputOptions.SpecificDeviceInfo.GetValue().Id : nullptr;
		const FTCHARToUTF8 Utf8DeviceId(DeviceIdPtr);

		EOS_RTCAudio_SetAudioInputSettingsOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_SETAUDIOINPUTSETTINGS_API_LATEST, 1);
		Options.LocalUserId = LoginSession.LocalUserProductUserId;
		Options.DeviceId = Utf8DeviceId.Get();
		Options.Volume = AudioInputOptions.Volume * 50.0;
		Options.bPlatformAEC = AudioInputOptions.bPlatformAEC ? EOS_TRUE : EOS_FALSE;

		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
		QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_UpdateAudioInput);
		const EOS_EResult EosResult = EOS_RTCAudio_SetAudioInputSettings(EOS_RTC_GetAudioInterface(GetRtcInterface()), &Options);
		if (EosResult != EOS_EResult::EOS_Success)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("ApplyAudioInputOptions failed: %s"), *LexToString(EosResult));
		}
	}
}

void FEOSVoiceChatUser::ApplyAudioOutputOptions()
{
	if (IsLoggedIn())
	{
		const TCHAR* DeviceIdPtr = AudioOutputOptions.SpecificDeviceInfo.IsSet() ?
			*AudioOutputOptions.SpecificDeviceInfo.GetValue().Id : nullptr;
		const FTCHARToUTF8 Utf8DeviceId(DeviceIdPtr);

		EOS_RTCAudio_SetAudioOutputSettingsOptions Options = {};
		Options.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_SETAUDIOOUTPUTSETTINGS_API_LATEST, 1);
		Options.LocalUserId = LoginSession.LocalUserProductUserId;
		Options.DeviceId = Utf8DeviceId.Get();
		Options.Volume = AudioOutputOptions.bMuted ? 0.0f : AudioOutputOptions.Volume * 50.0;

		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
		QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_UpdateAudioOutput);
		const EOS_EResult EosResult = EOS_RTCAudio_SetAudioOutputSettings(EOS_RTC_GetAudioInterface(GetRtcInterface()), &Options);
		if (EosResult != EOS_EResult::EOS_Success)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("ApplyAudioOutputOptions failed: %s"), *LexToString(EosResult));
		}
	}
}

void FEOSVoiceChatUser::ApplyPlayerBlock(const FGlobalParticipant& GlobalParticipant, const FChannelSession& ChannelSession, FChannelParticipant& ChannelParticipant)
{
	EOSVOICECHATUSER_LOG(Verbose, TEXT("ApplyPlayerBlock ChannelName=[%s] PlayerName=[%s] bBlocked=%s"), *ChannelSession.ChannelName, *GlobalParticipant.PlayerName, *LexToString(GlobalParticipant.bBlocked));

	const FTCHARToUTF8 Utf8RoomName(*ChannelSession.ChannelName);
	const EOS_ProductUserId ParticipantId = EOSProductUserIdFromString(*GlobalParticipant.PlayerName);

	EOS_RTC_BlockParticipantOptions Options = {};
	Options.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTC_BLOCKPARTICIPANT_API_LATEST, 1);
	Options.LocalUserId = LoginSession.LocalUserProductUserId;
	Options.RoomName = Utf8RoomName.Get();
	Options.ParticipantId = ParticipantId;
	Options.bBlocked = GlobalParticipant.bBlocked;

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_BlockParticipant);
	EOS_RTC_BlockParticipant(GetRtcInterface(), &Options, this, &FEOSVoiceChatUser::OnBlockParticipantStatic);
};

void FEOSVoiceChatUser::ApplyReceivingOptions(const FChannelSession& ChannelSession)
{
	const FTCHARToUTF8 Utf8RoomName(*ChannelSession.ChannelName);

	EOS_RTCAudio_UpdateReceivingOptions UpdateReceivingOptions = {};
	UpdateReceivingOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UPDATERECEIVING_API_LATEST, 1);
	UpdateReceivingOptions.LocalUserId = LoginSession.LocalUserProductUserId;
	UpdateReceivingOptions.RoomName = Utf8RoomName.Get();
	UpdateReceivingOptions.ParticipantId = nullptr;
	UpdateReceivingOptions.bAudioEnabled = ChannelSession.bIsNotListening ? EOS_FALSE : EOS_TRUE;

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_UpdateReceiving);
	EOS_RTCAudio_UpdateReceiving(EOS_RTC_GetAudioInterface(GetRtcInterface()), &UpdateReceivingOptions, this, &FEOSVoiceChatUser::OnUpdateReceivingAudioStatic);
}

void FEOSVoiceChatUser::ApplyPlayerReceivingOptions(const FString& PlayerName)
{
	check(IsLoggedIn());

	FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);

	for (TPair<FString, FChannelSession>& Pair : LoginSession.ChannelSessions)
	{
		FChannelSession& ChannelSession = Pair.Value;
		if (FChannelParticipant* ChannelParticipant = ChannelSession.Participants.Find(PlayerName))
		{
			ApplyPlayerReceivingOptions(GlobalParticipant, ChannelSession, *ChannelParticipant);
		}
	}
}

void FEOSVoiceChatUser::ApplyPlayerReceivingOptions(const FGlobalParticipant& GlobalParticipant, const FChannelSession& ChannelSession, FChannelParticipant& ChannelParticipant)
{
	// TODO ChannelParticipant should be const, and we should update the bAudioMuted (the *actual* state, not the *desired* state) in the OnUpdateReceiving callback.
	check(IsLoggedIn());

	ChannelParticipant.bAudioDisabled = GlobalParticipant.bAudioMuted || ChannelSession.bIsNotListening || ChannelParticipant.bMutedInChannel;
	EOSVOICECHATUSER_LOG(Verbose, TEXT("ApplyPlayerReceivingOptions ChannelName=[%s] PlayerName=[%s] bAudioMuted=[%s]"), *ChannelSession.ChannelName, *GlobalParticipant.PlayerName, *LexToString(GlobalParticipant.bAudioMuted));

	const FTCHARToUTF8 Utf8RoomName(*ChannelSession.ChannelName);
	const EOS_ProductUserId ParticipantId = EOSProductUserIdFromString(*GlobalParticipant.PlayerName);

	{
		EOS_RTCAudio_UpdateReceivingOptions UpdateReceivingOptions = {};
		UpdateReceivingOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UPDATERECEIVING_API_LATEST, 1);
		UpdateReceivingOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		UpdateReceivingOptions.RoomName = Utf8RoomName.Get();
		UpdateReceivingOptions.ParticipantId = ParticipantId;
		UpdateReceivingOptions.bAudioEnabled = ChannelParticipant.bAudioDisabled ? EOS_FALSE : EOS_TRUE;

		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
		QUICK_SCOPE_CYCLE_COUNTER(EOS_RTCAudio_UpdateReceiving);
		EOS_RTCAudio_UpdateReceiving(EOS_RTC_GetAudioInterface(GetRtcInterface()), &UpdateReceivingOptions, this, &FEOSVoiceChatUser::OnUpdateReceivingAudioStatic);
	}

	{
		EOS_RTCAudio_UpdateParticipantVolumeOptions UpdateParticipantVolumeOptions = {};
		UpdateParticipantVolumeOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UPDATEPARTICIPANTVOLUME_API_LATEST, 1);
		UpdateParticipantVolumeOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		UpdateParticipantVolumeOptions.RoomName = Utf8RoomName.Get();
		UpdateParticipantVolumeOptions.ParticipantId = ParticipantId;
		UpdateParticipantVolumeOptions.Volume = GlobalParticipant.Volume * 50.f;

		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
		QUICK_SCOPE_CYCLE_COUNTER(EOS_RTCAudio_UpdateParticipantVolume);
		EOS_RTCAudio_UpdateParticipantVolume(EOS_RTC_GetAudioInterface(GetRtcInterface()), &UpdateParticipantVolumeOptions, this, &FEOSVoiceChatUser::OnUpdateParticipantVolumeStatic);
	}
}

void FEOSVoiceChatUser::ApplySendingOptions()
{
	for (TPair<FString, FChannelSession>& Pair : LoginSession.ChannelSessions)
	{
		FChannelSession& ChannelSession = Pair.Value;
		if (ChannelSession.JoinState == EChannelJoinState::Joined)
		{
			ApplySendingOptions(ChannelSession);
		}
	}
}

void FEOSVoiceChatUser::ApplySendingOptions(FChannelSession& ChannelSession)
{
	const bool bCanTransmitToChannel = TransmitState.Mode == EVoiceChatTransmitMode::All
	|| (TransmitState.Mode == EVoiceChatTransmitMode::SpecificChannels && TransmitState.SpecificChannels.Contains(ChannelSession.ChannelName));

	ChannelSession.DesiredSendingState.bAudioEnabled = bCanTransmitToChannel && !AudioInputOptions.bMuted && !ChannelSession.bIsNotListening;

	const FTCHARToUTF8 Utf8RoomName(*ChannelSession.ChannelName);
		
	EOS_RTCAudio_UpdateSendingOptions UpdateSendingOptions = {};
	UpdateSendingOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_UPDATESENDING_API_LATEST, 1);
	UpdateSendingOptions.LocalUserId = LoginSession.LocalUserProductUserId;
	UpdateSendingOptions.RoomName = Utf8RoomName.Get();
	UpdateSendingOptions.AudioStatus = ChannelSession.bIsNotListening
		? EOS_ERTCAudioStatus::EOS_RTCAS_NotListeningDisabled
		: ChannelSession.DesiredSendingState.bAudioEnabled
		? EOS_ERTCAudioStatus::EOS_RTCAS_Enabled
		: EOS_ERTCAudioStatus::EOS_RTCAS_Disabled;

	// Protect against the callback firing after this object is destroyed.
	FUpdateSendingAudioCallback* Callback = new FUpdateSendingAudioCallback(AsWeak());
	Callback->CallbackLambda = [this](const EOS_RTCAudio_UpdateSendingCallbackInfo* Data) { OnUpdateSendingAudio(Data); };

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTCAudio_UpdateSending);
	EOS_RTCAudio_UpdateSending(EOS_RTC_GetAudioInterface(GetRtcInterface()), &UpdateSendingOptions, Callback, Callback->GetCallbackPtr());
}

void FEOSVoiceChatUser::BindLoginCallbacks()
{
	// OnLobbyChannelConnectionChanged
	{
		EOS_Lobby_AddNotifyRTCRoomConnectionChangedOptions Options = {};
		Options.ApiVersion = 2;
		UE_EOS_CHECK_API_MISMATCH(EOS_LOBBY_ADDNOTIFYRTCROOMCONNECTIONCHANGED_API_LATEST, 2);
		LoginSession.OnLobbyChannelConnectionChangedNotificationId = EOS_Lobby_AddNotifyRTCRoomConnectionChanged(GetLobbyInterface(), &Options, this, &FEOSVoiceChatUser::OnLobbyChannelConnectionChangedStatic);
		if (LoginSession.OnLobbyChannelConnectionChangedNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindLoginCallbacks EOS_Lobby_AddNotifyRTCRoomConnectionChanged failed"));
		}
	}
}

void FEOSVoiceChatUser::UnbindLoginCallbacks()
{
	if (LoginSession.OnLobbyChannelConnectionChangedNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_Lobby_RemoveNotifyRTCRoomConnectionChanged(GetLobbyInterface(), LoginSession.OnLobbyChannelConnectionChangedNotificationId);
		LoginSession.OnLobbyChannelConnectionChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	}
}

void FEOSVoiceChatUser::BindChannelCallbacks(FChannelSession& ChannelSession)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("BindChannelCallbacks ChannelName=[%s]"), *ChannelSession.ChannelName);

	const FTCHARToUTF8 Utf8RoomName(*ChannelSession.ChannelName);
	const FTCHARToUTF8 Utf8LobbyId(*ChannelSession.LobbyId);

	// OnChannelDisconnected
	if(!ChannelSession.IsLobbySession())
	{
		EOS_RTC_AddNotifyDisconnectedOptions DisconnectedOptions = {};
		DisconnectedOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTC_ADDNOTIFYDISCONNECTED_API_LATEST, 1);
		DisconnectedOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		DisconnectedOptions.RoomName = Utf8RoomName.Get();
		ChannelSession.OnChannelDisconnectedNotificationId = EOS_RTC_AddNotifyDisconnected(GetRtcInterface(), &DisconnectedOptions, this, &FEOSVoiceChatUser::OnChannelDisconnectedStatic);
		if (ChannelSession.OnChannelDisconnectedNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTC_AddNotifyDisconnected failed"));
		}
	}

	// OnParticipantStatusChanged
	{
		EOS_RTC_AddNotifyParticipantStatusChangedOptions ParticipantStatusChangedOptions = {};
		ParticipantStatusChangedOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTC_ADDNOTIFYPARTICIPANTSTATUSCHANGED_API_LATEST, 1);
		ParticipantStatusChangedOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		ParticipantStatusChangedOptions.RoomName = Utf8RoomName.Get();
		ChannelSession.OnParticipantStatusChangedNotificationId = EOS_RTC_AddNotifyParticipantStatusChanged(GetRtcInterface(), &ParticipantStatusChangedOptions, this, &FEOSVoiceChatUser::OnChannelParticipantStatusChangedStatic);
		if (ChannelSession.OnParticipantStatusChangedNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTC_AddNotifyParticipantStatusChanged failed"));
		}
	}

	// OnParticipantAudioUpdated
	{
		EOS_RTCAudio_AddNotifyParticipantUpdatedOptions ParticipantAudioUpdatedOptions = {};
		ParticipantAudioUpdatedOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYPARTICIPANTUPDATED_API_LATEST, 1);
		ParticipantAudioUpdatedOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		ParticipantAudioUpdatedOptions.RoomName = Utf8RoomName.Get();
		ChannelSession.OnParticipantAudioUpdatedNotificationId = EOS_RTCAudio_AddNotifyParticipantUpdated(EOS_RTC_GetAudioInterface(GetRtcInterface()), &ParticipantAudioUpdatedOptions, this, &FEOSVoiceChatUser::OnChannelParticipantAudioUpdatedStatic);
		if (ChannelSession.OnParticipantAudioUpdatedNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTCAudio_AddNotifyParticipantUpdated failed"));
		}
	}

	// OnAudioBeforeSend
	{
		EOS_RTCAudio_AddNotifyAudioBeforeSendOptions AudioBeforeSendOptions = {};
		AudioBeforeSendOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYAUDIOBEFORESEND_API_LATEST, 1);
		AudioBeforeSendOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		AudioBeforeSendOptions.RoomName = Utf8RoomName.Get();

		// Protect against callbacks occurring after this object is destroyed, by wrapping in a TEOSGlobalCallback. This can occur when LeaveRoom during Logout fails.
		TUniquePtr<FAudioBeforeSendCallback> Callback = MakeUnique<FAudioBeforeSendCallback>(AsWeak());
		Callback->CallbackLambda = [this](const EOS_RTCAudio_AudioBeforeSendCallbackInfo* Data) { OnChannelAudioBeforeSend(Data); };
		Callback->bIsGameThreadCallback = false;

		ChannelSession.OnAudioBeforeSendNotificationId = EOS_RTCAudio_AddNotifyAudioBeforeSend(EOS_RTC_GetAudioInterface(GetRtcInterface()), &AudioBeforeSendOptions, Callback.Get(), Callback->GetCallbackPtr());
		if (ChannelSession.OnAudioBeforeSendNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTCAudio_AddNotifyAudioBeforeSend failed"));
		}
		else
		{
			ChannelSession.AudioBeforeSendCallback = MoveTemp(Callback);
		}
	}
		
	// OnAudioBeforeRender (Mixed)
	{
		EOS_RTCAudio_AddNotifyAudioBeforeRenderOptions AudioBeforeRenderOptions = {};
		AudioBeforeRenderOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYAUDIOBEFORERENDER_API_LATEST, 1);
		AudioBeforeRenderOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		AudioBeforeRenderOptions.RoomName = Utf8RoomName.Get();
		AudioBeforeRenderOptions.bUnmixedAudio = false;
		ChannelSession.OnMixedAudioBeforeRenderNotificationId = EOS_RTCAudio_AddNotifyAudioBeforeRender(EOS_RTC_GetAudioInterface(GetRtcInterface()), &AudioBeforeRenderOptions, this, &FEOSVoiceChatUser::OnChannelMixedAudioBeforeRenderStatic);
		if (ChannelSession.OnMixedAudioBeforeRenderNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTCAudio_AddNotifyAudioBeforeRender (Mixed) failed"));
		}
	}
	
	// OnAudioBeforeRender (Unmixed)
	{
		EOS_RTCAudio_AddNotifyAudioBeforeRenderOptions AudioBeforeRenderOptions = {};
		AudioBeforeRenderOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYAUDIOBEFORERENDER_API_LATEST, 1);
		AudioBeforeRenderOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		AudioBeforeRenderOptions.RoomName = Utf8RoomName.Get();
		AudioBeforeRenderOptions.bUnmixedAudio = true;
		ChannelSession.OnUnmixedAudioBeforeRenderNotificationId = EOS_RTCAudio_AddNotifyAudioBeforeRender(EOS_RTC_GetAudioInterface(GetRtcInterface()), &AudioBeforeRenderOptions, this, &FEOSVoiceChatUser::OnChannelUnmixedAudioBeforeRenderStatic);
		if (ChannelSession.OnUnmixedAudioBeforeRenderNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTCAudio_AddNotifyAudioBeforeRender (Unmixed) failed"));
		}
	}

	// OnAudioInputState
	{
		EOS_RTCAudio_AddNotifyAudioInputStateOptions AudioInputStateOptions = {};
		AudioInputStateOptions.ApiVersion = 1;
		UE_EOS_CHECK_API_MISMATCH(EOS_RTCAUDIO_ADDNOTIFYAUDIOINPUTSTATE_API_LATEST, 1);
		AudioInputStateOptions.LocalUserId = LoginSession.LocalUserProductUserId;
		AudioInputStateOptions.RoomName = Utf8RoomName.Get();
		ChannelSession.OnAudioInputStateNotificationId = EOS_RTCAudio_AddNotifyAudioInputState(EOS_RTC_GetAudioInterface(GetRtcInterface()), &AudioInputStateOptions, this, &FEOSVoiceChatUser::OnChannelAudioInputStateStatic);
		if (ChannelSession.OnAudioInputStateNotificationId == EOS_INVALID_NOTIFICATIONID)
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("BindChannelCallbacks EOS_RTCAudio_AddNotifyAudioInputState failed"));
		}
	}
}

void FEOSVoiceChatUser::UnbindChannelCallbacks(FChannelSession& ChannelSession)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("UnbindChannelCallbacks ChannelName=[%s]"), *ChannelSession.ChannelName);

	if (ChannelSession.OnChannelDisconnectedNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTC_RemoveNotifyDisconnected(GetRtcInterface(), ChannelSession.OnChannelDisconnectedNotificationId);
		ChannelSession.OnChannelDisconnectedNotificationId = EOS_INVALID_NOTIFICATIONID;
	}

	if (ChannelSession.OnParticipantStatusChangedNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTC_RemoveNotifyParticipantStatusChanged(GetRtcInterface(), ChannelSession.OnParticipantStatusChangedNotificationId);
		ChannelSession.OnParticipantStatusChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	}

	if (ChannelSession.OnParticipantAudioUpdatedNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyParticipantUpdated(EOS_RTC_GetAudioInterface(GetRtcInterface()), ChannelSession.OnParticipantAudioUpdatedNotificationId);
		ChannelSession.OnParticipantAudioUpdatedNotificationId = EOS_INVALID_NOTIFICATIONID;
	}

	if (ChannelSession.OnAudioBeforeSendNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyAudioBeforeSend(EOS_RTC_GetAudioInterface(GetRtcInterface()), ChannelSession.OnAudioBeforeSendNotificationId);
		ChannelSession.OnAudioBeforeSendNotificationId = EOS_INVALID_NOTIFICATIONID;

		if (ChannelSession.AudioBeforeSendCallback)
		{
			if (bInDestructor)
			{
				// After an OnLeaveRoom callback, it is guaranteed this callback will no longer be fired. This method is in almost all cases called
				// in response to an OnLeaveRoom callback, so it's safe to release it here. The one exception, is if we are being destroyed. In that
				// case, the callback may be fired even after unbinding, so we move ownership of the callback to EOSShared so it can outlive this,
				// and be cleaned up eventually after EOS_Shutdown.
				EOSVoiceChat.SDKManager.AddCallbackObject(MoveTemp(ChannelSession.AudioBeforeSendCallback));
			}
			ChannelSession.AudioBeforeSendCallback.Reset();
		}
	}

	if (ChannelSession.OnMixedAudioBeforeRenderNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyAudioBeforeRender(EOS_RTC_GetAudioInterface(GetRtcInterface()), ChannelSession.OnMixedAudioBeforeRenderNotificationId);
		ChannelSession.OnMixedAudioBeforeRenderNotificationId = EOS_INVALID_NOTIFICATIONID;
	}
	
	if (ChannelSession.OnUnmixedAudioBeforeRenderNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyAudioBeforeRender(EOS_RTC_GetAudioInterface(GetRtcInterface()), ChannelSession.OnUnmixedAudioBeforeRenderNotificationId);
		ChannelSession.OnUnmixedAudioBeforeRenderNotificationId = EOS_INVALID_NOTIFICATIONID;
	}

	if (ChannelSession.OnAudioInputStateNotificationId != EOS_INVALID_NOTIFICATIONID)
	{
		EOS_RTCAudio_RemoveNotifyAudioInputState(EOS_RTC_GetAudioInterface(GetRtcInterface()), ChannelSession.OnAudioInputStateNotificationId);
		ChannelSession.OnAudioInputStateNotificationId = EOS_INVALID_NOTIFICATIONID;
	}
}

void FEOSVoiceChatUser::LeaveChannelInternal(const FString& ChannelName, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate)
{
	FChannelSession& ChannelSession = GetChannelSession(ChannelName);
	ChannelSession.JoinState = EChannelJoinState::Leaving;
	ChannelSession.LeaveDelegates = { Delegate };

	const FTCHARToUTF8 Utf8RoomName(*ChannelName);

	EOS_RTC_LeaveRoomOptions LeaveOptions = {};
	LeaveOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_RTC_LEAVEROOM_API_LATEST, 1);
	LeaveOptions.LocalUserId = LoginSession.LocalUserProductUserId;
	LeaveOptions.RoomName = Utf8RoomName.Get();

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(EOS_RTC_LeaveRoom);
	EOS_RTC_LeaveRoom(GetRtcInterface(), &LeaveOptions, this, &FEOSVoiceChatUser::OnLeaveRoomStatic);
}

void FEOSVoiceChatUser::LogoutInternal(const FOnVoiceChatLogoutCompleteDelegate& Delegate)
{
	if (ensure(LoginSession.State == ELoginState::LoggedIn || LoginSession.State == ELoginState::LoggingOut))
	{
		if(LoginSession.State == ELoginState::LoggingOut)
		{
			check(LoginSession.LogoutState.IsSet());
			LoginSession.LogoutState->CompletionDelegates.Emplace(Delegate);
		}
		else
		{
			LoginSession.State = ELoginState::LoggingOut;

			// We have to actually leave all the channels, because there is no concept of "connected" or "logged in" in RTC, so no API calls we can call to "leave everything" like in vivox
			TSet<FString> JoinedChannelNames;
			TSet<FString> LeavingChannelNames;
			TSet<FString> LobbyIds;
			{
				for (const TPair<FString, FChannelSession>& Pair : LoginSession.ChannelSessions)
				{
					const FChannelSession& ChannelSession = Pair.Value;
					if (ChannelSession.IsLobbySession())
					{
						LobbyIds.Add(ChannelSession.LobbyId);
					}
					else if (ChannelSession.JoinState >= EChannelJoinState::Joining)
					{
						JoinedChannelNames.Add(ChannelSession.ChannelName);
					}
					else if (ChannelSession.JoinState == EChannelJoinState::Leaving)
					{
						LeavingChannelNames.Add(ChannelSession.ChannelName);
					}
				}
			}

			for (const FString& LobbyId : LobbyIds)
			{
				RemoveLobbyRoom(LobbyId);
			}

			FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
			if (JoinedChannelNames.Num() > 0 || LeavingChannelNames.Num() > 0)
			{
				// Channels are connected, so we fire leave requests in parallel, wait for them to complete, then return success/failure
				FLoginSession::FLogoutState& LogoutState = LoginSession.LogoutState.Emplace();
				LogoutState.CompletionDelegates.Emplace(Delegate);
				LogoutState.ChannelNamesExpectingCallback.Reserve(JoinedChannelNames.Num() + LeavingChannelNames.Num());
				LogoutState.ChannelNamesExpectingCallback.Append(JoinedChannelNames);
				LogoutState.ChannelNamesExpectingCallback.Append(LeavingChannelNames);

				FOnVoiceChatChannelLeaveCompleteDelegate LeaveCompleteDelegate = FOnVoiceChatChannelLeaveCompleteDelegate::CreateLambda([this](const FString& ChannelName, const FVoiceChatResult& LambdaResult)
				{
					if (!LambdaResult.IsSuccess())
					{
						EOSVOICECHATUSER_LOG(Warning, TEXT("LogoutInternal LeaveChannelInternal ChannelName=[%s] %s"), *ChannelName, *LexToString(LambdaResult));
						LoginSession.LogoutState->Result = LambdaResult;
					}
					LoginSession.LogoutState->ChannelNamesExpectingCallback.Remove(ChannelName);
					if (LoginSession.LogoutState->ChannelNamesExpectingCallback.Num() == 0)
					{
						const FVoiceChatResult Result = MoveTemp(LoginSession.LogoutState->Result);
						const TArray<FOnVoiceChatLogoutCompleteDelegate> LogoutCompleteDelegates = MoveTemp(LoginSession.LogoutState->CompletionDelegates);
						if (Result.IsSuccess())
						{
							// All the LeaveChannel requests succeeded, so clear the login session
							const FString PlayerName = MoveTemp(LoginSession.PlayerName);
							ClearLoginSession();

							for (const FOnVoiceChatLogoutCompleteDelegate& Delegate : LogoutCompleteDelegates)
							{
								Delegate.ExecuteIfBound(PlayerName, FVoiceChatResult::CreateSuccess());
							}
							OnVoiceChatLoggedOutDelegate.Broadcast(PlayerName);
						}
						else
						{
							// One of the LeaveChannel requests failed, so we stay logged in
							LoginSession.State = ELoginState::LoggedIn;
							LoginSession.LogoutState.Reset();

							for (const FOnVoiceChatLogoutCompleteDelegate& Delegate : LogoutCompleteDelegates)
							{
								Delegate.ExecuteIfBound(LoginSession.PlayerName, Result);
							}
						}
					}
				});

				for (const FString& ChannelName : JoinedChannelNames)
				{
					// Actually leave joined/joining channels
					LeaveChannelInternal(ChannelName, LeaveCompleteDelegate);
				}

				for (const FString& ChannelName : LeavingChannelNames)
				{
					// Add to the LeaveDelegates of already leaving channels
					GetChannelSession(ChannelName).LeaveDelegates.Add(LeaveCompleteDelegate);
				}
			}
			else
			{
				// No channels connected, so just clear the LoginSession and fire successes
				const FString PlayerName = MoveTemp(LoginSession.PlayerName);
				ClearLoginSession();

				Delegate.ExecuteIfBound(PlayerName, FVoiceChatResult::CreateSuccess());
				OnVoiceChatLoggedOutDelegate.Broadcast(PlayerName);
			}
		}
	}
}

void FEOSVoiceChatUser::ClearLoginSession()
{
	for (TPair<FString, FChannelSession>& Pair : LoginSession.ChannelSessions)
	{
		UnbindChannelCallbacks(Pair.Value);
	}
	UnbindLoginCallbacks();
	LoginSession = FLoginSession();
}

#pragma region EOSCallbacks
void EOS_CALL FEOSVoiceChatUser::OnJoinRoomStatic(const EOS_RTC_JoinRoomCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnJoinRoom(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnJoinRoomStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnJoinRoomStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnJoinRoom(const EOS_RTC_JoinRoomCallbackInfo* CallbackInfo)
{
	if (!IsInitialized())
	{
		// This is expected behavior - when the client tries to leave from a room before received JoinRoomCompletion, 
		// JoinRoomCompletion is invoked with EResult::EOS_RTC_RoomWasLeft(in the majority) + LeaveRoomCompletion(EResult::Success).
		return;
	}

	FVoiceChatResult Result = ResultFromEOSResult(CallbackInfo->ResultCode);
	FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);

	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		const FString& ParticipantId = ChannelSession->PlayerName;
		if (ChannelSession->JoinState == EChannelJoinState::Joining)
		{
			if (Result.IsSuccess())
			{
				EOSVOICECHATUSER_LOG(Log, TEXT("OnJoinRoom ChannelName=[%s] Result=[%s] RoomOptions=[%s]"),
					*ChannelName,
					*LexToString(Result),
					*FString::JoinBy(TArrayView<const EOS_RTC_Option>(CallbackInfo->RoomOptions, CallbackInfo->RoomOptionsCount), TEXT(", "), UE_PROJECTION(LexToString))
				);

				ChannelSession->JoinState = EChannelJoinState::Joined;
				TriggerCompletionDelegate(ChannelSession->JoinDelegate, ChannelSession->ChannelName, FVoiceChatResult::CreateSuccess());
				OnVoiceChatChannelJoinedDelegate.Broadcast(ChannelSession->ChannelName);

				FChannelParticipant& ChannelParticipant = ChannelSession->Participants.Add(ChannelSession->PlayerName);
				ChannelParticipant.PlayerName = ChannelSession->PlayerName;
				OnVoiceChatPlayerAddedDelegate.Broadcast(ChannelSession->ChannelName, ChannelSession->PlayerName);

				ApplySendingOptions(*ChannelSession);
			}
			else
			{
				EOSVOICECHATUSER_LOG(Warning, TEXT("OnJoinRoom ChannelName=[%s] Result=[%s]"), *ChannelName, *LexToString(Result));

				ChannelSession->JoinState = EChannelJoinState::NotJoined;

				TriggerCompletionDelegate(ChannelSession->JoinDelegate, ChannelSession->ChannelName, Result);
				RemoveChannelSession(ChannelSession->ChannelName);
			}
		}
		else if (ChannelSession->JoinState == EChannelJoinState::Leaving)
		{
			// We've already called LeaveRoom to "cancel" the JoinRoom, so we're expecting a callback for that.
			EOSVOICECHATUSER_LOG(Log, TEXT("OnJoinRoom ChannelName=[%s] Result=[%s] Ignoring due to State=Leaving"), *ChannelName, *LexToString(Result));
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnJoinRoom ChannelName=[%s] Result=[%s] Unexpected State=%s"), *ChannelName, *LexToString(Result), LexToString(ChannelSession->JoinState));
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnJoinRoom ChannelName=[%s] not found"), *ChannelName);
	}
}

void EOS_CALL FEOSVoiceChatUser::OnLeaveRoomStatic(const EOS_RTC_LeaveRoomCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnLeaveRoom(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnLeaveRoomStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnLeaveRoomStatic Error CallbackInfo=nullptr")
	}
}

void FEOSVoiceChatUser::OnLeaveRoom(const EOS_RTC_LeaveRoomCallbackInfo* CallbackInfo)
{
	check(LoginSession.State == ELoginState::LoggedIn
		|| LoginSession.State == ELoginState::LoggingOut);

	FVoiceChatResult Result = ResultFromEOSResult(CallbackInfo->ResultCode);
	FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);

	if (Result.IsSuccess())
	{
		EOSVOICECHATUSER_LOG(Log, TEXT("OnLeaveRoom ChannelName=[%s] Result=[%s]"), *ChannelName, *LexToString(Result));
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnLeaveRoom ChannelName=[%s] Result=[%s]"), *ChannelName, *LexToString(Result));
	}

	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		const bool bWasLeaving = ChannelSession->JoinState == EChannelJoinState::Leaving;

		if (Result.IsSuccess())
		{
			ChannelSession->JoinState = EChannelJoinState::NotJoined;
		}
		else
		{
			ChannelSession->JoinState = EChannelJoinState::Joined;
		}

		if (bWasLeaving)
		{
			TriggerCompletionDelegates(ChannelSession->LeaveDelegates, ChannelSession->ChannelName, Result);
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnLeaveRoom ChannelName=[%s] unexpected, not in Leaving state"), *ChannelName);
		}

		if (Result.IsSuccess())
		{
			// ChannelSession may have been removed by the completion delegate
			OnVoiceChatChannelExitedDelegate.Broadcast(ChannelName, Result);

			RemoveChannelSession(ChannelName);
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnLeaveRoom ChannelName=[%s] not found"), *ChannelName);
	}
}

void EOS_CALL FEOSVoiceChatUser::OnBlockParticipantStatic(const EOS_RTC_BlockParticipantCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnBlockParticipant(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnBlockUserStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnBlockUserStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnBlockParticipant(const EOS_RTC_BlockParticipantCallbackInfo* CallbackInfo)
{
	check(IsInitialized());

	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const FString PlayerName = LexToString(CallbackInfo->ParticipantId);

	if (CallbackInfo->ResultCode == EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(Log, TEXT("OnBlockUser ChannelName=[%s] PlayerName=[%s] Block=[%s] Success"), *ChannelName, *PlayerName, *LexToString(CallbackInfo->bBlocked));
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnBlockUser ChannelName=[%s] PlayerName=[%s] Block=[%s] failed error=[%s]"), *ChannelName, *PlayerName, *LexToString(CallbackInfo->bBlocked), *LexToString(CallbackInfo->ResultCode));
	}
}

void EOS_CALL FEOSVoiceChatUser::OnUpdateParticipantVolumeStatic(const EOS_RTCAudio_UpdateParticipantVolumeCallbackInfo* CallbackInfo)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EOSVoiceChat);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEOSVoiceChat_OnUpdateParticipantVolumeStatic);

	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnUpdateParticipantVolume(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnUpdateParticipantVolumeStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnUpdateParticipantVolumeStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnUpdateParticipantVolume(const EOS_RTCAudio_UpdateParticipantVolumeCallbackInfo* CallbackInfo)
{
	check(IsInitialized());

	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const FString PlayerName = LexToString(CallbackInfo->ParticipantId);
	const EOS_EResult& ResultCode = CallbackInfo->ResultCode;

	if (ResultCode == EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnUpdateParticipantVolume ChannelName=[%s] PlayerName=[%s] Success"), *ChannelName, *PlayerName);
		FChannelSession& ChannelSession = GetChannelSession(ChannelName);
		if (FChannelParticipant* ChannelParticipant = ChannelSession.Participants.Find(PlayerName))
		{
			const FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
			OnVoiceChatPlayerVolumeUpdatedDelegate.Broadcast(ChannelSession.ChannelName, ChannelParticipant->PlayerName, GlobalParticipant.Volume);
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateParticipantVolume ChannelName=[%s] PlayerName=[%s] failed to find player in channel."), *ChannelName, *PlayerName);
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateParticipantVolume ChannelName=[%s] PlayerName=[%s] failed error=[%s]"), *ChannelName, *PlayerName, *LexToString(ResultCode));
	}
}

void EOS_CALL FEOSVoiceChatUser::OnUpdateReceivingAudioStatic(const EOS_RTCAudio_UpdateReceivingCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnUpdateReceivingAudio(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnUpdateReceivingStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnUpdateReceivingStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnUpdateReceivingAudio(const EOS_RTCAudio_UpdateReceivingCallbackInfo* CallbackInfo)
{
	check(IsInitialized());

	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const FString PlayerName = LexToString(CallbackInfo->ParticipantId);
	const EOS_EResult& ResultCode = CallbackInfo->ResultCode;

	if (ResultCode == EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnUpdateReceiving ChannelName=[%s] PlayerName=[%s] Success"), *ChannelName, *PlayerName);

		// UpdateReceiving is called once prior to JoinRoom to set the default receiving state, in which case PlayerName is empty
		if (!PlayerName.IsEmpty())
		{
			FChannelSession& ChannelSession = GetChannelSession(ChannelName);
			if (FChannelParticipant* ChannelParticipant = ChannelSession.Participants.Find(PlayerName))
			{
				const FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
				OnVoiceChatPlayerMuteUpdatedDelegate.Broadcast(ChannelSession.ChannelName, ChannelParticipant->PlayerName, GlobalParticipant.bAudioMuted);
			}
			else
			{
				EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateReceiving ChannelName=[%s] PlayerName=[%s] failed to find player in channel."), *ChannelName, *PlayerName);
			}
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateReceiving ChannelName=[%s] PlayerName=[%s] failed error=[%s]"), *ChannelName, *PlayerName, *LexToString(ResultCode));
	}
}

void FEOSVoiceChatUser::OnUpdateSendingAudio(const EOS_RTCAudio_UpdateSendingCallbackInfo* CallbackInfo)
{
	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const bool bAudioEnabled = CallbackInfo->AudioStatus == EOS_ERTCAudioStatus::EOS_RTCAS_Enabled;

	if (CallbackInfo->ResultCode == EOS_EResult::EOS_Success)
	{
		EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnUpdateSending ChannelName=[%s] Success"), *ChannelName);

		if (IsLoggedIn())
		{
			if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
			{
				if (bAudioEnabled != ChannelSession->ActiveSendingState.bAudioEnabled)
				{
					ChannelSession->ActiveSendingState.bAudioEnabled = bAudioEnabled;

					if (FChannelParticipant* LocalChannelParticipant = ChannelSession->Participants.Find(ChannelSession->PlayerName))
					{
						// The "Talking" state exposed via IVoiceChat is bTalking && bAudioEnabled. If bTalking == false there is no change.
						if (LocalChannelParticipant->bTalking)
						{
							FGlobalParticipant& LocalGlobalParticipant = GetGlobalParticipant(LocalChannelParticipant->PlayerName);
							LocalGlobalParticipant.bTalking = bAudioEnabled;
							OnVoiceChatPlayerTalkingUpdatedDelegate.Broadcast(ChannelSession->ChannelName, ChannelSession->PlayerName, LocalGlobalParticipant.bTalking);
						}
					}
					else
					{
						// Not a warning as the first call to UpdateSending is before JoinRoom, so before the local participant is added...
						EOSVOICECHATUSER_LOG(Verbose, TEXT("OnUpdateSending ChannelName=[%s] PlayerName=[%s] not found"), *ChannelName, *ChannelSession->PlayerName);
					}
				}
			}
			else
			{
				EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateSending ChannelName=[%s] not found"), *ChannelName);
			}
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnUpdateSending ChannelName=[%s] failed error=[%s]"), *ChannelName, *LexToString(CallbackInfo->ResultCode));
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelDisconnectedStatic(const EOS_RTC_DisconnectedCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelDisconnected(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelDisconnectedStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelDisconnectedStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnChannelDisconnected(const EOS_RTC_DisconnectedCallbackInfo* CallbackInfo)
{
	FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	FVoiceChatResult Result = ResultFromEOSResult(CallbackInfo->ResultCode);

	EOSVOICECHATUSER_LOG(Log, TEXT("OnChannelDisconnected ChannelName=[%s] Result=[%s]"), *ChannelName, *LexToString(Result));

	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		if (ChannelSession->JoinState == EChannelJoinState::Leaving)
		{
			EOSVOICECHATUSER_LOG(Verbose, TEXT("OnChannelDisconnected ChannelName=[%s] State=Leaving waiting for LeaveRoom callback"), *ChannelName);
		}
		else
		{
			ChannelSession->JoinState = EChannelJoinState::NotJoined;
					
			OnVoiceChatChannelExitedDelegate.Broadcast(ChannelName, Result);

			RemoveChannelSession(ChannelName);
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelDisconnected ChannelName=[%s] not found"), *ChannelName);
	}
}

void EOS_CALL FEOSVoiceChatUser::OnLobbyChannelConnectionChangedStatic(const EOS_Lobby_RTCRoomConnectionChangedCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnLobbyChannelConnectionChanged(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnLobbyChannelConnectionChangedStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnLobbyChannelConnectionChangedStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnLobbyChannelConnectionChanged(const EOS_Lobby_RTCRoomConnectionChangedCallbackInfo* CallbackInfo)
{
	const EOS_ProductUserId LocalUserId = CallbackInfo->LocalUserId;
	const FString LobbyId = UTF8_TO_TCHAR(CallbackInfo->LobbyId);
	const bool bIsConnected = CallbackInfo->bIsConnected == EOS_TRUE;
	const FVoiceChatResult Reason = ResultFromEOSResult(CallbackInfo->DisconnectReason);

	if (LocalUserId != LoginSession.LocalUserProductUserId)
	{
		EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnLobbyChannelConnectionChanged LocalUserId=[%s] LobbyId=[%s] bIsConnected=[%s] Reason=[%s] Skipping notification for other user"),
			*LexToString(LocalUserId), *LobbyId, *LexToString(bIsConnected), *LexToString(Reason));
	}

	EOSVOICECHATUSER_LOG(Log, TEXT("OnLobbyChannelConnectionChanged LobbyId=[%s] bIsConnected=[%s] Reason=[%s]"), *LobbyId, *LexToString(bIsConnected), *LexToString(Reason));

	if (FString* ChannelName = LoginSession.LobbyIdToChannelName.Find(LobbyId))
	{
		FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(*ChannelName);
		if(ensure(ChannelSession))
		{
			ChannelSession->bLobbyChannelConnected = bIsConnected;

			// TODO fire connection changed delegate
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnLobbyChannelConnectionChanged LobbyId=[%s] not found"), *LobbyId);
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelParticipantStatusChangedStatic(const EOS_RTC_ParticipantStatusChangedCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelParticipantStatusChanged(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelParticipantStatusChangedStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelParticipantStatusChangedStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnChannelParticipantStatusChanged(const EOS_RTC_ParticipantStatusChangedCallbackInfo* CallbackInfo)
{
	check(LoginSession.State == ELoginState::LoggedIn);

	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const FString PlayerName = LexToString(CallbackInfo->ParticipantId);

	if (CallbackInfo->ParticipantStatus == EOS_ERTCParticipantStatus::EOS_RTCPS_Joined)
	{
		if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
		{
			if (PlayerName != ChannelSession->PlayerName)
			{
				EOSVOICECHATUSER_LOG(Verbose, TEXT("OnChannelParticipantJoined ChannelName=[%s] PlayerName=[%s]"), *ChannelName, *PlayerName);

				const FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
				FChannelParticipant& ChannelParticipant = ChannelSession->Participants.Add(PlayerName);
				ChannelParticipant.PlayerName = PlayerName;

				TArray<FVoiceChatMetadataItem> Metadata;
				for (uint32 i = 0; i < CallbackInfo->ParticipantMetadataCount; ++i)
				{
					FString Key(UTF8_TO_TCHAR(CallbackInfo->ParticipantMetadata[i].Key));
					FString Value(UTF8_TO_TCHAR(CallbackInfo->ParticipantMetadata[i].Value));
					Metadata.Emplace(FVoiceChatMetadataItem{ MoveTemp(Key), MoveTemp(Value) });
				}

				OnVoiceChatPlayerAddedDelegate.Broadcast(ChannelName, PlayerName);
				FEOSVoiceChatDelegates::OnVoiceChatPlayerAddedMetadata.Broadcast(LoginSession.PlayerName, ChannelName, PlayerName, Metadata);

				ApplyPlayerReceivingOptions(GlobalParticipant, *ChannelSession, ChannelParticipant);
				ApplyPlayerBlock(GlobalParticipant, *ChannelSession, ChannelParticipant);
			}
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelParticipantJoined ChannelName=[%s] not found"), *ChannelName);
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Verbose, TEXT("OnChannelParticipantLeft ChannelName=[%s] PlayerName=[%s]"), *ChannelName, *PlayerName);

		if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
		{
			ChannelSession->Participants.Remove(PlayerName);
			OnVoiceChatPlayerRemovedDelegate.Broadcast(ChannelSession->ChannelName, PlayerName);
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelParticipantLeft ChannelName=[%s] not found"), *ChannelName);
		}
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelParticipantAudioUpdatedStatic(const EOS_RTCAudio_ParticipantUpdatedCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelParticipantAudioUpdated(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelParticipantAudioUpdatedStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelParticipantAudioUpdatedStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnChannelParticipantAudioUpdated(const EOS_RTCAudio_ParticipantUpdatedCallbackInfo* CallbackInfo)
{
	check(LoginSession.State == ELoginState::LoggedIn);

	const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);
	const FString PlayerName = LexToString(CallbackInfo->ParticipantId);

	const bool bTalking = CallbackInfo->bSpeaking == EOS_TRUE;
	const EOS_ERTCAudioStatus AudioStatus = CallbackInfo->AudioStatus;

	if (FChannelSession* ChannelSession = LoginSession.ChannelSessions.Find(ChannelName))
	{
		if (FChannelParticipant* ChannelParticipant = ChannelSession->Participants.Find(PlayerName))
		{
			if (ChannelParticipant->bTalking != bTalking)
			{
				EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnChannelParticipantAudioUpdated ChannelName=[%s] PlayerName=[%s] bTalking=[%s]"), *ChannelName, *PlayerName, *LexToString(bTalking));
				ChannelParticipant->bTalking = bTalking;

				FGlobalParticipant& GlobalParticipant = GetGlobalParticipant(PlayerName);
				const bool bIsThisUser = PlayerName == ChannelSession->PlayerName;
				if (!bIsThisUser
					// For the local user, the "Talking" state exposed via IVoiceChat is bTalking && bAudioEnabled. If bAudioEnabled == false there is no change.
					|| ChannelSession->ActiveSendingState.bAudioEnabled)
				{
					GlobalParticipant.bTalking = bTalking;
					OnVoiceChatPlayerTalkingUpdatedDelegate.Broadcast(ChannelSession->ChannelName, PlayerName, GlobalParticipant.bTalking);
				}
			}

			if (!ChannelParticipant->AudioStatus.IsSet() || ChannelParticipant->AudioStatus.GetValue() != AudioStatus)
			{
				EOSVOICECHATUSER_LOG(VeryVerbose, TEXT("OnChannelParticipantAudioUpdated ChannelName=[%s] PlayerName=[%s] AudioStatus=[%s]"), *ChannelName, *PlayerName, *LexToString(AudioStatus));
				ChannelParticipant->AudioStatus.Emplace(AudioStatus);
				FEOSVoiceChatDelegates::OnAudioStatusChanged.Broadcast(LoginSession.PlayerName, ChannelSession->ChannelName, PlayerName, AudioStatus);
			}
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelParticipantAudioUpdated ChannelName=[%s] PlayerName=[%s] not found"), *ChannelName, *PlayerName);
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelParticipantAudioUpdated ChannelName=[%s] not found"), *ChannelName);
	}
}

void FEOSVoiceChatUser::OnChannelAudioBeforeSend(const EOS_RTCAudio_AudioBeforeSendCallbackInfo* CallbackInfo)
{
	if (EOS_RTCAudio_AudioBuffer* Buffer = CallbackInfo->Buffer)
	{
		if (Buffer->Frames)
		{
			TArrayView<const int16> Samples = MakeArrayView(Buffer->Frames, Buffer->FramesCount * Buffer->Channels);
			TArrayView<int16> WriteableSamples = MakeArrayView(Buffer->Frames, Buffer->FramesCount * Buffer->Channels);

			// TODO EOS doesn't tell us if it's silence or not, maybe need to compare all the samples to some threshold?
			const bool bSpeaking = true;

			if (bFakeAudioInput)
			{
				static float Envelope = 0x7fff;  // int16 samples
				static float freq = 440.0f;
				for (uint32 FrameIndex = 0; FrameIndex < Buffer->FramesCount; FrameIndex++)
				{
					float SineOutput = Envelope * FMath::Sin((2 * PI * freq * FrameIndex) / (float)Buffer->SampleRate);
					for (uint32 ChannelIndex = 0; ChannelIndex < Buffer->Channels; ChannelIndex++)
					{
						WriteableSamples[FrameIndex * Buffer->Channels + ChannelIndex] = SineOutput;
					}
				}
			}

			const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);

			FScopeLock Lock(&BeforeCaptureAudioSentLock);

			// Allow any processes to modify audio through DSP effects processing
			OnVoiceChatAfterCaptureAudioReadDelegate.Broadcast(ChannelName, WriteableSamples, Buffer->SampleRate, Buffer->Channels);

			OnVoiceChatBeforeCaptureAudioSentDelegate.Broadcast(ChannelName, Samples, Buffer->SampleRate, Buffer->Channels, bSpeaking);

		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelAudioBeforeRender Error Frames=nullptr"));
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelAudioBeforeRender Error Buffer=nullptr"));
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelMixedAudioBeforeRenderStatic(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelMixedAudioBeforeRender(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelMixedAudioBeforeRenderStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelMixedAudioBeforeRenderStatic Error CallbackInfo=nullptr");
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelUnmixedAudioBeforeRenderStatic(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelUnmixedAudioBeforeRender(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelUnmixedAudioBeforeRenderStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelUnmixedAudioBeforeRenderStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnChannelMixedAudioBeforeRender(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo)
{
	if (EOS_RTCAudio_AudioBuffer* Buffer = CallbackInfo->Buffer)
	{
		if (Buffer->Frames)
		{
			TArrayView<int16> Samples = MakeArrayView(Buffer->Frames, Buffer->FramesCount * Buffer->Channels);

			// TODO EOS doesn't tell us if it's silence or not, maybe need to compare all the samples to some threshold?
			const bool bIsSilence = false;
			const FString PlayerName = LexToString(CallbackInfo->ParticipantId);
			const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);

			FScopeLock Lock(&BeforeRecvMixedAudioRenderedLock);
			OnVoiceChatBeforeRecvMixedAudioRenderedDelegate.Broadcast(Samples, Buffer->SampleRate, Buffer->Channels, bIsSilence, ChannelName, PlayerName);
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelMixedAudioBeforeRender Error Frames=nullptr"));
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelMixedAudioBeforeRender Error Buffer=nullptr"));
	}
}

void FEOSVoiceChatUser::OnChannelUnmixedAudioBeforeRender(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo)
{
	if (EOS_RTCAudio_AudioBuffer* Buffer = CallbackInfo->Buffer)
	{
		if (Buffer->Frames)
		{
			TArrayView<int16> Samples = MakeArrayView(Buffer->Frames, Buffer->FramesCount * Buffer->Channels);

			// TODO EOS doesn't tell us if it's silence or not, maybe need to compare all the samples to some threshold?
			const bool bIsSilence = false;
			const FString PlayerName = LexToString(CallbackInfo->ParticipantId);
			const FString ChannelName = UTF8_TO_TCHAR(CallbackInfo->RoomName);

			FScopeLock Lock(&BeforeRecvUnmixedAudioRenderedLock);
			OnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate.Broadcast(Samples, Buffer->SampleRate, Buffer->Channels, bIsSilence, ChannelName, PlayerName);
		}
		else
		{
			EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelUnmixedAudioBeforeRender Error Frames=nullptr"));
		}
	}
	else
	{
		EOSVOICECHATUSER_LOG(Warning, TEXT("OnChannelUnmixedAudioBeforeRender Error Buffer=nullptr"));
	}
}

void EOS_CALL FEOSVoiceChatUser::OnChannelAudioInputStateStatic(const EOS_RTCAudio_AudioInputStateCallbackInfo* CallbackInfo)
{
	if (CallbackInfo)
	{
		if (FEOSVoiceChatUser* EosVoiceChatPtr = static_cast<FEOSVoiceChatUser*>(CallbackInfo->ClientData))
		{
			EosVoiceChatPtr->OnChannelAudioInputState(CallbackInfo);
		}
		else
		{
			UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelAudioInputStateStatic Error EosVoiceChatPtr=nullptr");
		}
	}
	else
	{
		UE_LOGF(LogEOSVoiceChat, Warning, "OnChannelAudioInputStateStatic Error CallbackInfo=nullptr");
	}
}

void FEOSVoiceChatUser::OnChannelAudioInputState(const EOS_RTCAudio_AudioInputStateCallbackInfo* CallbackInfo)
{
	EOSVOICECHATUSER_LOG(Log, TEXT("OnChannelAudioInputState ChannelName=[%s] State=[%s]"), UTF8_TO_TCHAR(CallbackInfo->RoomName), LexToString(CallbackInfo->Status));

	FEOSVoiceChatDelegates::OnAudioInputDeviceStatusChanged.Broadcast(LoginSession.PlayerName, CallbackInfo->Status);
}

#pragma endregion EOSCallbacks

bool FEOSVoiceChatUser::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !NO_LOGGING
#define EOS_EXEC_LOG(Fmt, ...) Ar.CategorizedLogf(LogEOSVoiceChat.GetCategoryName(), ELogVerbosity::Log, Fmt, ##__VA_ARGS__)
#else
#define EOS_EXEC_LOG(Fmt, ...) 
#endif
	if (FParse::Command(&Cmd, TEXT("INFO")))
	{
		if (IsInitialized())
		{
			EOS_EXEC_LOG(TEXT("    Input Devices: muted:%s volume:%.2f aec:%s"), *LexToString(GetAudioInputDeviceMuted()), GetAudioInputVolume(), *LexToString(AudioInputOptions.bPlatformAEC));
			const FVoiceChatDeviceInfo InputDevice = GetInputDeviceInfo();
			const FVoiceChatDeviceInfo DefaultInputDevice = GetDefaultInputDeviceInfo();
			if (InputDevice == DefaultInputDevice)
			{
				EOS_EXEC_LOG(TEXT("      [%s] (Selected) (Default)"), *LexToString(DefaultInputDevice));
			}
			else
			{
				EOS_EXEC_LOG(TEXT("      [%s] (Selected)"), *LexToString(InputDevice));
				EOS_EXEC_LOG(TEXT("      [%s] (Default)"), *LexToString(DefaultInputDevice));
			}
			for (const FVoiceChatDeviceInfo& Device : GetAvailableInputDeviceInfos())
			{
				if (Device != DefaultInputDevice && Device != InputDevice)
				{
					EOS_EXEC_LOG(TEXT("      [%s]"), *LexToString(Device));
				}
			}

			EOS_EXEC_LOG(TEXT("    Output Devices: muted:%s volume:%.2f"), *LexToString(GetAudioOutputDeviceMuted()), GetAudioOutputVolume());
			const FVoiceChatDeviceInfo OutputDevice = GetOutputDeviceInfo();
			const FVoiceChatDeviceInfo DefaultOutputDevice = GetDefaultOutputDeviceInfo();
			if (OutputDevice == DefaultOutputDevice)
			{
				EOS_EXEC_LOG(TEXT("      [%s] (Selected) (Default)"), *LexToString(DefaultOutputDevice));
			}
			else
			{
				EOS_EXEC_LOG(TEXT("      [%s] (Selected)"), *LexToString(OutputDevice));
				EOS_EXEC_LOG(TEXT("      [%s] (Default)"), *LexToString(DefaultOutputDevice));
			}
			for (const FVoiceChatDeviceInfo& Device : GetAvailableOutputDeviceInfos())
			{
				if (Device != DefaultOutputDevice && Device != OutputDevice)
				{
					EOS_EXEC_LOG(TEXT("      [%s]"), *LexToString(Device));
				}
			}

			if (IsConnected())
			{
				EOS_EXEC_LOG(TEXT("    Login Status: %s"), LexToString(LoginSession.State));
				if (IsLoggedIn())
				{
					EOS_EXEC_LOG(TEXT("      PlayerName: %s"), *LoginSession.PlayerName);

					FString TransmitString;
					switch (GetTransmitMode())
					{
					case EVoiceChatTransmitMode::All:
						TransmitString = TEXT("ALL");
						break;
					case EVoiceChatTransmitMode::None:
						TransmitString = TEXT("NONE");
						break;
					case EVoiceChatTransmitMode::SpecificChannels:
						TransmitString = FString::Printf(TEXT("CHANNELS:[%s]"), *FString::Join(GetTransmitChannels(), TEXT(", ")));
						break;
					}
					EOS_EXEC_LOG(TEXT("    Channels: transmitting:%s"), *TransmitString);
					for (const TPair<FString, FChannelSession>& ChannelSessionPair : LoginSession.ChannelSessions)
					{
						const FString& ChannelName = ChannelSessionPair.Key;
						const FChannelSession& ChannelSession = ChannelSessionPair.Value;
						EOS_EXEC_LOG(TEXT("      %s"), *ChannelName);
						EOS_EXEC_LOG(TEXT("        PlayerName: %s"), *ChannelSession.PlayerName);
						EOS_EXEC_LOG(TEXT("        Join Status: %s"), LexToString(ChannelSession.JoinState));
						EOS_EXEC_LOG(TEXT("        bIsNotListeningDisabled: %s"), *LexToString(ChannelSession.bIsNotListening));
						EOS_EXEC_LOG(TEXT("        bAudioEnabled: Desired:%s Actual:%s"), *LexToString(ChannelSession.DesiredSendingState.bAudioEnabled), *LexToString(ChannelSession.ActiveSendingState.bAudioEnabled));
						EOS_EXEC_LOG(TEXT("        bIsLobby: %s"), *LexToString(ChannelSession.IsLobbySession()));
						if (ChannelSession.IsLobbySession())
						{
							EOS_EXEC_LOG(TEXT("          LobbyId: %s"), *ChannelSession.LobbyId);
							EOS_EXEC_LOG(TEXT("          bLobbyChannelConnected: %s"), *LexToString(ChannelSession.bLobbyChannelConnected));
						}
						EOS_EXEC_LOG(TEXT("        Participants:"));
						for (const TPair<FString, FChannelParticipant>& ParticipantPair : ChannelSession.Participants)
						{
							const FString& ParticipantName = ParticipantPair.Key;
							const FChannelParticipant& Participant = ParticipantPair.Value;
							EOS_EXEC_LOG(TEXT("          %s talking:%s bAudioDisabled:%s"), *ParticipantName, *LexToString(Participant.bTalking), *LexToString(Participant.bAudioDisabled));
						}
					}
				}
			}
		}
		return true;
	}
#if !UE_BUILD_SHIPPING
	else if (FParse::Command(&Cmd, TEXT("INPUT")))
	{
		if (FParse::Command(&Cmd, TEXT("SETVOLUME")))
		{
			FString Volume;
			if (FParse::Value(Cmd, TEXT("Volume="), Volume))
			{
				SetAudioInputVolume(FCString::Atof(*Volume));
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("MUTE")))
		{
			SetAudioInputDeviceMuted(true);
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("UNMUTE")))
		{
			SetAudioInputDeviceMuted(false);
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("LISTDEVICES")))
		{
			EOS_EXEC_LOG(TEXT("Input Devices:"));
			for (const FVoiceChatDeviceInfo& Device : GetAvailableInputDeviceInfos())
			{
				EOS_EXEC_LOG(TEXT("  %s"), *LexToString(Device));
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("SETDEVICE")))
		{
			FString DeviceId;
			if (FParse::Value(Cmd, TEXT("DeviceId="), DeviceId))
			{
				SetInputDeviceId(DeviceId);
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("SETDEFAULTDEVICE")))
		{
			SetInputDeviceId(FString());
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("OUTPUT")))
	{
		if (FParse::Command(&Cmd, TEXT("SETVOLUME")))
		{
			FString Volume;
			if (FParse::Value(Cmd, TEXT("Volume="), Volume))
			{
				SetAudioOutputVolume(FCString::Atof(*Volume));
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("MUTE")))
		{
			SetAudioOutputDeviceMuted(true);
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("UNMUTE")))
		{
			SetAudioOutputDeviceMuted(false);
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("LISTDEVICES")))
		{
			EOS_EXEC_LOG(TEXT("Output Devices:"));
			for (const FVoiceChatDeviceInfo& Device : GetAvailableOutputDeviceInfos())
			{
				EOS_EXEC_LOG(TEXT("  %s"), *LexToString(Device));
			}
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("SETDEVICE")))
		{
			FString DeviceId;
			if (FParse::Value(Cmd, TEXT("DeviceId="), DeviceId))
			{
				SetOutputDeviceId(DeviceId);
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("SETDEFAULTDEVICE")))
		{
			SetOutputDeviceId(FString());
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("LOGIN")))
	{
		FString PlayerName;
		if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
		{
			FString Token = InsecureGetLoginToken(PlayerName);

			FPlatformUserId FirstUser = FPlatformMisc::GetPlatformUserForUserIndex(0);
			Login(FirstUser, PlayerName, Token, FOnVoiceChatLoginCompleteDelegate::CreateLambda([this, &Ar](const FString& LoggedInPlayerName, const FVoiceChatResult& Result)
			{
				EOSVOICECHATUSER_LOG(Display, TEXT("EOS LOGIN playername:%s result:%s"), *LoggedInPlayerName, *LexToString(Result));
			}));
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("LOGOUT")))
	{
		Logout(FOnVoiceChatLogoutCompleteDelegate::CreateLambda([this, &Ar](const FString& PlayerName, const FVoiceChatResult& Result)
		{
			EOSVOICECHATUSER_LOG(Display, TEXT("EOS LOGOUT playername:%s result:%s"), *PlayerName, *LexToString(Result));
		}));
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("CHANNEL")))
	{
		if (FParse::Command(&Cmd, TEXT("JOIN")))
		{
			FString ChannelName;
			if (FParse::Value(Cmd, TEXT("ChannelName="), ChannelName))
			{
				FString ChannelTypeString;
				EVoiceChatChannelType ChannelType = EVoiceChatChannelType::NonPositional;
				TOptional<FVoiceChatChannel3dProperties> Channel3dProperties;
				if (FParse::Value(Cmd, TEXT("ChannelType="), ChannelTypeString))
				{
					if (ChannelTypeString.Compare(TEXT("POSITIONAL"), ESearchCase::IgnoreCase) == 0)
					{
						ChannelType = EVoiceChatChannelType::Positional;
					}
					else if (ChannelTypeString.Compare(TEXT("ECHO"), ESearchCase::IgnoreCase) == 0)
					{
						ChannelType = EVoiceChatChannelType::Echo;
					}
				}

				FString Token = InsecureGetJoinToken(ChannelName, ChannelType, Channel3dProperties);

				JoinChannel(ChannelName, Token, ChannelType, FOnVoiceChatChannelJoinCompleteDelegate::CreateLambda([this, &Ar](const FString& JoinedChannelName, const FVoiceChatResult& Result)
				{
					EOSVOICECHATUSER_LOG(Display, TEXT("EOS CHANNEL JOIN channelname:%s result:%s"), *JoinedChannelName, *LexToString(Result));
				}), Channel3dProperties);
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("LEAVE")))
		{
			FString ChannelName;
			if (FParse::Value(Cmd, TEXT("ChannelName="), ChannelName))
			{
				LeaveChannel(ChannelName, FOnVoiceChatChannelLeaveCompleteDelegate::CreateLambda([this, &Ar](const FString& LeftChannelName, const FVoiceChatResult& Result)
				{
					EOSVOICECHATUSER_LOG(Display, TEXT("EOS CHANNEL LEAVE channelname:%s result:%s"), *LeftChannelName, *LexToString(Result));
				}));
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("TRANSMIT")))
		{
			FString ChannelName;
			if (FParse::Value(Cmd, TEXT("ChannelName="), ChannelName))
			{
				TransmitToSpecificChannels({ ChannelName });
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("TRANSMITALL")))
		{
			TransmitToAllChannels();
			return true;
		}
		else if (FParse::Command(&Cmd, TEXT("TRANSMITNONE")))
		{
			TransmitToNoChannels();
			return true;
		}
	}
	else if (FParse::Command(&Cmd, TEXT("PLAYER")))
	{
		if (FParse::Command(&Cmd, TEXT("MUTE")))
		{
			FString PlayerName;
			if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
			{
				SetPlayerMuted(PlayerName, true);
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("UNMUTE")))
		{
			FString PlayerName;
			if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
			{
				SetPlayerMuted(PlayerName, false);
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("SETVOLUME")))
		{
			FString PlayerName;
			if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
			{
				FString Volume;
				if (FParse::Value(Cmd, TEXT("Volume="), Volume))
				{
					SetPlayerVolume(PlayerName, FCString::Atof(*Volume));
					return true;
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("BLOCK")))
		{
			FString PlayerName;
			if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
			{
				BlockPlayers({ PlayerName });
				return true;
			}
		}
		else if (FParse::Command(&Cmd, TEXT("UNBLOCK")))
		{
			FString PlayerName;
			if (FParse::Value(Cmd, TEXT("PlayerName="), PlayerName))
			{
				UnblockPlayers({ PlayerName });
				return true;
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

#undef EOS_EXEC_LOG

	return false;
}

FEOSVoiceChatUser::FChannelSession::~FChannelSession()
{
	check(OnChannelDisconnectedNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnParticipantStatusChangedNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnParticipantAudioUpdatedNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnAudioBeforeSendNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnMixedAudioBeforeRenderNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnUnmixedAudioBeforeRenderNotificationId == EOS_INVALID_NOTIFICATIONID);
	check(OnAudioInputStateNotificationId == EOS_INVALID_NOTIFICATIONID);
}

bool FEOSVoiceChatUser::FChannelSession::IsLocalUser(const FChannelParticipant& Participant)
{
	return PlayerName == Participant.PlayerName;
}

bool FEOSVoiceChatUser::FChannelSession::IsLobbySession() const
{
	return !LobbyId.IsEmpty();
}

FEOSVoiceChatUser::FLoginSession::~FLoginSession()
{
	check(OnLobbyChannelConnectionChangedNotificationId == EOS_INVALID_NOTIFICATIONID);
}

const TCHAR* LexToString(FEOSVoiceChatUser::ELoginState State)
{
	switch (State)
	{
	case FEOSVoiceChatUser::ELoginState::LoggedOut:		return TEXT("LoggedOut");
	case FEOSVoiceChatUser::ELoginState::LoggingOut:	return TEXT("LoggingOut");
	case FEOSVoiceChatUser::ELoginState::LoggingIn:		return TEXT("LoggingIn");
	case FEOSVoiceChatUser::ELoginState::LoggedIn:		return TEXT("LoggedIn");
	default:
		checkNoEntry();
		return TEXT("Unknown");
	}
}

const TCHAR* LexToString(FEOSVoiceChatUser::EChannelJoinState State)
{
	switch (State)
	{
	case FEOSVoiceChatUser::EChannelJoinState::NotJoined:	return TEXT("NotJoined");
	case FEOSVoiceChatUser::EChannelJoinState::Leaving:		return TEXT("Leaving");
	case FEOSVoiceChatUser::EChannelJoinState::Joining:		return TEXT("Joining");
	case FEOSVoiceChatUser::EChannelJoinState::Joined:		return TEXT("Joined");
	default:
		checkNoEntry();
		return TEXT("Unknown");
	}
}

#endif // WITH_EOSVOICECHAT
