// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSEOSVoiceChat.h"

#if WITH_EOSVOICECHAT

#include "IOSEOSVoiceChatUser.h"

#include "IOS/IOSAppDelegate.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

FIOSEOSVoiceChat::FIOSEOSVoiceChat(IEOSSDKManager& InSDKManager)
	: FEOSVoiceChat(InSDKManager)
{
}

void FIOSEOSVoiceChat::Initialize(const FOnVoiceChatInitializeCompleteDelegate& Delegate)
{
	FEOSVoiceChat::Initialize(FOnVoiceChatInitializeCompleteDelegate::CreateLambda([this, Delegate](const FVoiceChatResult& Result)
	{
		if (Result.IsSuccess())
		{
			GConfig->GetBool(TEXT("EOSVoiceChat"), TEXT("bEnableHardwareAEC"), bEnableHardwareAEC, GEngineIni);
			GConfig->GetBool(TEXT("EOSVoiceChat"), TEXT("bEnableBluetoothMicrophone"), bEnableBluetoothMicrophone, GEngineIni);

			if (!ApplicationDidEnterForegroundHandle.IsValid())
			{
				ApplicationDidEnterForegroundHandle = FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FIOSEOSVoiceChat::HandleApplicationHasEnteredForeground);
			}
			if (!AudioRouteChangedHandle.IsValid())
			{
				AudioRouteChangedHandle = FCoreDelegates::AudioRouteChangedDelegate.AddRaw(this, &FIOSEOSVoiceChat::HandleAudioRouteChanged);
			}
		}

		Delegate.ExecuteIfBound(Result);
	}));
}

void FIOSEOSVoiceChat::Uninitialize(const FOnVoiceChatUninitializeCompleteDelegate& Delegate)
{
	if (ApplicationDidEnterForegroundHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasReactivatedDelegate.Remove(ApplicationDidEnterForegroundHandle);
		ApplicationDidEnterForegroundHandle.Reset();
	}
	if (AudioRouteChangedHandle.IsValid())
	{
		FCoreDelegates::AudioRouteChangedDelegate.Remove(AudioRouteChangedHandle);
		AudioRouteChangedHandle.Reset();
	}

	FEOSVoiceChat::Uninitialize(FOnVoiceChatUninitializeCompleteDelegate::CreateLambda([this, Delegate](const FVoiceChatResult& Result)
	{
		if (Result.IsSuccess())
		{
			while (VoiceChatEnableCount > 0)
			{
				EnableVoiceChat(false);
			}
		}

		Delegate.ExecuteIfBound(Result);
	}));
}

IVoiceChatUser* FIOSEOSVoiceChat::CreateUser()
{
	const FEOSVoiceChatUserRef& User = VoiceChatUsers.Emplace_GetRef(MakeShared<FIOSEOSVoiceChatUser, ESPMode::ThreadSafe>(*this));
	return &User.Get();
}

bool FIOSEOSVoiceChat::IsHardwareAECEnabled() const
{
	return OverrideEnableHardwareAEC.Get(bEnableHardwareAEC);
}

void FIOSEOSVoiceChat::SetHardwareAECEnabled(bool bEnabled)
{
	OverrideEnableHardwareAEC = bEnabled;
	UpdateVoiceChatSettings();
}

bool FIOSEOSVoiceChat::IsBluetoothMicrophoneEnabled() const
{
	return OverrideEnableBluetoothMicrophone.Get(bEnableBluetoothMicrophone);
}

void FIOSEOSVoiceChat::SetBluetoothMicrophoneEnabled(bool bEnabled)
{
	OverrideEnableBluetoothMicrophone = bEnabled;
	UpdateVoiceChatSettings();
}

void FIOSEOSVoiceChat::EnableVoiceChat(bool bEnable)
{
	if (bEnable)
	{
		++VoiceChatEnableCount;
		if (VoiceChatEnableCount == 1)
		{
			[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Playback Active:true];
			[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Record Active:true];

			const bool bEnableAEC = IsHardwareAECEnabled() && IsUsingBuiltInSpeaker();
			SendAECSettingToUsers(bEnableAEC);
		}
	}
	else
	{
		if (ensureMsgf(VoiceChatEnableCount > 0, TEXT("Attempted to disable voice chat when it was already disabled")))
		{
			--VoiceChatEnableCount;
			if (VoiceChatEnableCount == 0)
			{
				SendAECSettingToUsers(false);

				[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Record Active:false];
				[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::Playback Active:false];
			}
		}
	}
}

void FIOSEOSVoiceChat::HandleApplicationHasEnteredForeground()
{
	EOSVOICECHATUSER_LOG(Log, TEXT("OnApplicationHasEnteredForegoundDelegate"));

	// HandleAudioRouteChanged is not getting called when a route change happens in the background. Update voice chat settings here to handle this case
	UpdateVoiceChatSettings();
}

void FIOSEOSVoiceChat::HandleAudioRouteChanged(bool)
{
	EOSVOICECHATUSER_LOG(Verbose, TEXT("Audio route changed"));
	UpdateVoiceChatSettings();
}

void FIOSEOSVoiceChat::UpdateVoiceChatSettings()
{
	if (bBluetoothMicrophoneFeatureEnabled != IsBluetoothMicrophoneEnabled())
	{
		[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::BluetoothMicrophone Active:IsBluetoothMicrophoneEnabled()];
		bBluetoothMicrophoneFeatureEnabled = IsBluetoothMicrophoneEnabled();
	}

	const bool bEnableAEC = IsHardwareAECEnabled() && IsUsingBuiltInSpeaker();

	if (bVoiceChatModeEnabled != bEnableAEC)
	{
		[[IOSAppDelegate GetDelegate] SetFeature:EAudioFeature::VoiceChat Active:bEnableAEC];
		bVoiceChatModeEnabled = bEnableAEC;
	}

	if (VoiceChatEnableCount > 0)
	{
		EOSVOICECHATUSER_LOG(Verbose, TEXT("%s AEC"), bEnableAEC ? TEXT("Enabling") : TEXT("Disabling"));
		SendAECSettingToUsers(bEnableAEC);
	}
}

bool FIOSEOSVoiceChat::IsUsingBuiltInSpeaker()
{
	if (AVAudioSessionRouteDescription* CurrentRoute = [[AVAudioSession sharedInstance] currentRoute])
	{
		for (AVAudioSessionPortDescription* Port in[CurrentRoute outputs])
		{
			if ([[Port portType] isEqualToString:AVAudioSessionPortBuiltInReceiver]
				|| [[Port portType] isEqualToString:AVAudioSessionPortBuiltInSpeaker])
			{
				return true;
			}
		}
	}

	return false;
}

void FIOSEOSVoiceChat::SendAECSettingToUsers(bool bEnableAEC)
{
	if (SingleUserVoiceChatUser)
	{
		static_cast<FIOSEOSVoiceChatUser*>(SingleUserVoiceChatUser)->SetHardwareAECEnabled(bEnableAEC);
	}
	else
	{
		for (const FEOSVoiceChatUserRef& VoiceChatUser : VoiceChatUsers)
		{
			StaticCastSharedRef<FIOSEOSVoiceChatUser>(VoiceChatUser)->SetHardwareAECEnabled(bEnableAEC);
		}
	}
}

#endif // WITH_EOSVOICECHAT