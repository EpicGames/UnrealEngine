// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

class FIOSEOSVoiceChat : public FEOSVoiceChat
{
public:
	FIOSEOSVoiceChat(IEOSSDKManager& InSDKManager);
	virtual ~FIOSEOSVoiceChat() = default;

	// ~Begin IVoiceChat Interface
	virtual void Initialize(const FOnVoiceChatInitializeCompleteDelegate& Delegate) override;
	virtual void Uninitialize(const FOnVoiceChatUninitializeCompleteDelegate& Delegate) override;
	virtual IVoiceChatUser* CreateUser() override;
	// ~End IVoiceChat Interface

	bool IsHardwareAECEnabled() const;
	void SetHardwareAECEnabled(bool bEnabled);
	bool IsBluetoothMicrophoneEnabled() const;
	void SetBluetoothMicrophoneEnabled(bool bEnabled);
	void EnableVoiceChat(bool bEnable);

private:
	uint VoiceChatEnableCount = 0;

	bool bEnableHardwareAEC = false;
	TOptional<bool> OverrideEnableHardwareAEC;
	bool bVoiceChatModeEnabled = false;

	bool bEnableBluetoothMicrophone = false;
	TOptional<bool> OverrideEnableBluetoothMicrophone;
	bool bBluetoothMicrophoneFeatureEnabled = false;

	FDelegateHandle ApplicationWillEnterBackgroundHandle;
	FDelegateHandle ApplicationDidEnterForegroundHandle;
	FDelegateHandle AudioRouteChangedHandle;

	void HandleApplicationHasEnteredForeground();
	void HandleAudioRouteChanged(bool);

	void UpdateVoiceChatSettings();
	bool IsUsingBuiltInSpeaker();
	void SendAECSettingToUsers(bool bEnableAEC);
	
};

using FPlatformEOSVoiceChat = FIOSEOSVoiceChat;

#endif // WITH_EOSVOICECHAT