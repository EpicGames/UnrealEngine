// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSEOSVoiceChatUser.h"

#if WITH_EOSVOICECHAT

FIOSEOSVoiceChatUser::FIOSEOSVoiceChatUser(FEOSVoiceChat& InEOSVoiceChat)
	: FEOSVoiceChatUser(InEOSVoiceChat)
{
	OnVoiceChatChannelExited().AddRaw(this, &FIOSEOSVoiceChatUser::HandleVoiceChatChannelExited);
}

void FIOSEOSVoiceChatUser::SetSetting(const FString& Name, const FString& Value)
{
	if (Name == TEXT("HardwareAEC"))
	{
		GetIOSVoiceChat().SetHardwareAECEnabled(FCString::ToBool(*Value));
	}
	else if (Name == TEXT("BluetoothMicrophone"))
	{
		GetIOSVoiceChat().SetBluetoothMicrophoneEnabled(FCString::ToBool(*Value));
	}
	else
	{
		FEOSVoiceChatUser::SetSetting(Name, Value);
	}
}

void FIOSEOSVoiceChatUser::JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties)
{
	GetIOSVoiceChat().EnableVoiceChat(true);

	FOnVoiceChatChannelJoinCompleteDelegate DelegateWrapper = FOnVoiceChatChannelJoinCompleteDelegate::CreateLambda(
		[this, Delegate](const FString& ChannelName, const FVoiceChatResult& Result)
	{
		if (!Result.IsSuccess())
		{
			GetIOSVoiceChat().EnableVoiceChat(false);
		}
		Delegate.ExecuteIfBound(ChannelName, Result);
	});

	FEOSVoiceChatUser::JoinChannel(ChannelName, ChannelCredentials, ChannelType, DelegateWrapper, Channel3dProperties);
}

FIOSEOSVoiceChat& FIOSEOSVoiceChatUser::GetIOSVoiceChat()
{
	return static_cast<FIOSEOSVoiceChat&>(EOSVoiceChat);
}

void FIOSEOSVoiceChatUser::HandleVoiceChatChannelExited(const FString& ChannelName, const FVoiceChatResult& Reason)
{
	GetIOSVoiceChat().EnableVoiceChat(false);
}

#endif // WITH_EOSVOICECHAT