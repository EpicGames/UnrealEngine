// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChatUser.h"

#if WITH_EOSVOICECHAT

#include "IOSEOSVoiceChat.h"

class FIOSEOSVoiceChatUser : public FEOSVoiceChatUser
{
public:
	FIOSEOSVoiceChatUser(FEOSVoiceChat& InEOSVoiceChat);
	virtual ~FIOSEOSVoiceChatUser() = default;

	// ~Begin IVoiceChatUser
	virtual void SetSetting(const FString& Name, const FString& Value) override;
	virtual void JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties) override;
	// ~End IVoiceChatUSer

	FIOSEOSVoiceChat& GetIOSVoiceChat();

private:
    friend class FIOSEOSVoiceChat;
	void HandleVoiceChatChannelExited(const FString& ChannelName, const FVoiceChatResult& Reason);
};

#endif // WITH_EOSVOICECHAT