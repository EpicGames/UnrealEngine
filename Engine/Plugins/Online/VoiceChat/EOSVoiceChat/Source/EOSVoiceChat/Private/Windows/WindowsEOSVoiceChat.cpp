// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsEOSVoiceChat.h"

#if WITH_EOSVOICECHAT

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Windows/eos_Windows.h"

FWindowsEOSVoiceChat::FWindowsEOSVoiceChat(IEOSSDKManager& InSDKManager)
	: FEOSVoiceChat(InSDKManager)
{
}

IEOSPlatformHandlePtr FWindowsEOSVoiceChat::EOSPlatformCreate(EOS_Platform_Options& PlatformOptions)
{
	const FString XAudioPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/Windows/XAudio2_9/x64/xaudio2_9redist.dll")));
	const FTCHARToUTF8 Utf8XAudioPath(*XAudioPath);

	EOS_Windows_RTCOptions WindowsRTCOptions = { 0 };
	WindowsRTCOptions.ApiVersion = 1;
	UE_EOS_CHECK_API_MISMATCH(EOS_WINDOWS_RTCOPTIONS_API_LATEST, 1);
	WindowsRTCOptions.XAudio29DllPath = Utf8XAudioPath.Get();

	const_cast<EOS_Platform_RTCOptions*>(PlatformOptions.RTCOptions)->PlatformSpecificOptions = &WindowsRTCOptions;

	return FEOSVoiceChat::EOSPlatformCreate(PlatformOptions);
}

#endif // WITH_EOSVOICECHAT