// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

class FWindowsEOSVoiceChat : public FEOSVoiceChat
{
public:
	FWindowsEOSVoiceChat(IEOSSDKManager& InSDKManager);
	virtual ~FWindowsEOSVoiceChat() = default;

protected:
	virtual IEOSPlatformHandlePtr EOSPlatformCreate(EOS_Platform_Options& PlatformOptions) override;
};

using FPlatformEOSVoiceChat = FWindowsEOSVoiceChat;

#endif // WITH_EOSVOICECHAT