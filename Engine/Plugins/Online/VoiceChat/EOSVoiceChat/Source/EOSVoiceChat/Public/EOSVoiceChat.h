// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSShared.h"

#ifndef WITH_EOSVOICECHAT
#define WITH_EOSVOICECHAT WITH_EOS_RTC
#endif

#if WITH_EOSVOICECHAT

#include "Misc/CoreMisc.h"
#include "VoiceChat.h"

#include "eos_types.h"
#include "eos_rtc_types.h"
#include "eos_rtc_audio_types.h"
#include "eos_lobby_types.h"

struct FVoiceChatMetadataItem
{
	FString Key;
	FString Value;
};

class FEOSVoiceChatDelegates
{
public:
	/** Delegate called when the status of the audio device has changed. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAudioInputDeviceStatusChanged, FString /*PlayerName*/, EOS_ERTCAudioInputStatus /*Status*/);
	static EOSVOICECHAT_API FOnAudioInputDeviceStatusChanged OnAudioInputDeviceStatusChanged;
	/** Delegate called when ParticipantMetadata is included in the player joined event **/
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnVoiceChatPlayerAddedMetadataDelegate, const FString& /*LoginPlayerName*/, const FString& /* ChannelName */, const FString& /*PlayerName*/, const TArray<FVoiceChatMetadataItem>& /* PlayerMetadata */);
	static EOSVOICECHAT_API FOnVoiceChatPlayerAddedMetadataDelegate OnVoiceChatPlayerAddedMetadata;
	/** Delegate called when the audio status has changed **/
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnAudioStatusChanged, const FString& /*LoginPlayerName*/, const FString& /* ChannelName */, const FString& /*PlayerName*/, const EOS_ERTCAudioStatus /*Status*/);
	static EOSVOICECHAT_API FOnAudioStatusChanged OnAudioStatusChanged;
};

class FEOSAudioDevicePool;
class FEOSVoiceChatUser;
class IEOSSDKManager;
using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle, ESPMode::ThreadSafe>;

typedef TSharedPtr<class FEOSVoiceChat, ESPMode::ThreadSafe> FEOSVoiceChatPtr;
typedef TSharedPtr<class FEOSVoiceChatUser, ESPMode::ThreadSafe> FEOSVoiceChatUserPtr;
typedef TSharedRef<class FEOSVoiceChatUser, ESPMode::ThreadSafe> FEOSVoiceChatUserRef;

class FEOSVoiceChat : public TSharedFromThis<FEOSVoiceChat, ESPMode::ThreadSafe>, public IVoiceChat
{
public:
	FEOSVoiceChat(IEOSSDKManager& InSDKManager);

	UE_DEPRECATED(5.8, "Use FEOSVoiceChat(IEOSSDKManager&) and call SetPlatformHandle() instead")
	FEOSVoiceChat(IEOSSDKManager& InSDKManager, const IEOSPlatformHandlePtr& PlatformHandle);

	virtual ~FEOSVoiceChat();

	// ~Begin IVoiceChat Interface
	virtual bool Initialize() override;
	virtual void Initialize(const FOnVoiceChatInitializeCompleteDelegate& Delegate) override;
	virtual bool Uninitialize() override;
	virtual void Uninitialize(const FOnVoiceChatUninitializeCompleteDelegate& Delegate) override;
	virtual bool IsInitialized() const override;
	virtual void Connect(const FOnVoiceChatConnectCompleteDelegate& Delegate) override;
	virtual void Disconnect(const FOnVoiceChatDisconnectCompleteDelegate& Delegate) override;
	virtual bool IsConnecting() const override;
	virtual bool IsConnected() const override;
	virtual FOnVoiceChatConnectedDelegate& OnVoiceChatConnected() override { return OnVoiceChatConnectedDelegate; }
	virtual FOnVoiceChatDisconnectedDelegate& OnVoiceChatDisconnected() override { return OnVoiceChatDisconnectedDelegate; }
	virtual FOnVoiceChatReconnectedDelegate& OnVoiceChatReconnected() override { return OnVoiceChatReconnectedDelegate; }
	virtual IVoiceChatUser* CreateUser() override;
	virtual void ReleaseUser(IVoiceChatUser* VoiceChatUser) override;
	// ~End IVoiceChat Interface

	// Set the platform config name to use during Initialize(). Only used if no platform handle is set.
	void SetPlatformConfigName(const FString& PlatformConfigName) { OverridePlatformConfigName = PlatformConfigName; }

	// Set an existing platform handle to use during Initialize(). If set, Initialize() will use this
	// handle instead of creating one via named config or inline credentials
	void SetPlatformHandle(const IEOSPlatformHandlePtr& PlatformHandle) { EosPlatformHandle = PlatformHandle; }

	// ~Begin IVoiceChatUser Interface
	virtual void SetSetting(const FString& Name, const FString& Value) override;
	virtual FString GetSetting(const FString& Name) override;
	virtual void SetAudioInputVolume(float Volume) override;
	virtual void SetAudioOutputVolume(float Volume) override;
	virtual float GetAudioInputVolume() const override;
	virtual float GetAudioOutputVolume() const override;
	virtual void SetAudioInputDeviceMuted(bool bIsMuted) override;
	virtual void SetAudioOutputDeviceMuted(bool bIsMuted) override;
	virtual bool GetAudioInputDeviceMuted() const override;
	virtual bool GetAudioOutputDeviceMuted() const override;
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableInputDeviceInfos() const override;
	virtual TArray<FVoiceChatDeviceInfo> GetAvailableOutputDeviceInfos() const override;
	virtual FOnVoiceChatAvailableAudioDevicesChangedDelegate& OnVoiceChatAvailableAudioDevicesChanged() override;
	virtual void SetInputDeviceId(const FString& InputDeviceId) override;
	virtual void SetOutputDeviceId(const FString& OutputDeviceId) override;
	virtual FVoiceChatDeviceInfo GetInputDeviceInfo() const override;
	virtual FVoiceChatDeviceInfo GetOutputDeviceInfo() const override;
	virtual FVoiceChatDeviceInfo GetDefaultInputDeviceInfo() const override;
	virtual FVoiceChatDeviceInfo GetDefaultOutputDeviceInfo() const override;
	virtual void Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& Delegate) override;
	virtual void Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate) override;
	virtual bool IsLoggingIn() const override;
	virtual bool IsLoggedIn() const override;
	virtual FOnVoiceChatLoggedInDelegate& OnVoiceChatLoggedIn() override;
	virtual FOnVoiceChatLoggedOutDelegate& OnVoiceChatLoggedOut() override;
	virtual FString GetLoggedInPlayerName() const override;
	virtual void BlockPlayers(const TArray<FString>& PlayerNames) override;
	virtual void UnblockPlayers(const TArray<FString>& PlayerNames) override;
	virtual void JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override;
	virtual void LeaveChannel(const FString& Channel, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate) override;
	virtual FOnVoiceChatChannelJoinedDelegate& OnVoiceChatChannelJoined() override;
	virtual FOnVoiceChatChannelExitedDelegate& OnVoiceChatChannelExited() override;
	UE_DEPRECATED(5.8, "Please use OnVoiceChatCallStatsUpdated2 which adds the ChannelName param")
	virtual FOnVoiceChatCallStatsUpdatedDelegate& OnVoiceChatCallStatsUpdated() override;
	virtual FOnVoiceChatCallStatsUpdatedDelegate2& OnVoiceChatCallStatsUpdated2() override;
	virtual TOptional<FVoiceChatCallStats> GetChannelCallStats(const FString& ChannelName) const override;
	virtual void Set3DPosition(const FString& ChannelName, const FVector& Position) override;
	virtual TArray<FString> GetChannels() const override;
	virtual TArray<FString> GetPlayersInChannel(const FString& ChannelName) const override;
	virtual EVoiceChatChannelType GetChannelType(const FString& ChannelName) const override;
	virtual FOnVoiceChatPlayerAddedDelegate& OnVoiceChatPlayerAdded() override;
	virtual FOnVoiceChatPlayerRemovedDelegate& OnVoiceChatPlayerRemoved() override;
	virtual bool IsPlayerTalking(const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerTalkingUpdatedDelegate& OnVoiceChatPlayerTalkingUpdated() override;
	virtual void SetPlayerMuted(const FString& PlayerName, bool bMuted) override;
	virtual bool IsPlayerMuted(const FString& PlayerName) const override;
	virtual void SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bMuted) override;
	virtual bool IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerMuteUpdatedDelegate& OnVoiceChatPlayerMuteUpdated() override;
	virtual void SetPlayerVolume(const FString& PlayerName, float Volume) override;
	virtual float GetPlayerVolume(const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerVolumeUpdatedDelegate& OnVoiceChatPlayerVolumeUpdated() override;
	virtual void TransmitToAllChannels() override;
	virtual void TransmitToNoChannels() override;
	virtual void TransmitToSpecificChannels(const TSet<FString>& ChannelNames) override;
	virtual EVoiceChatTransmitMode GetTransmitMode() const override;
	virtual TSet<FString> GetTransmitChannels() const override;
	virtual FDelegateHandle StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate) override;
	virtual void StopRecording(FDelegateHandle Handle) override;
	virtual FDelegateHandle RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate) override;
	virtual void UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle) override;
	virtual FDelegateHandle RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate) override;
	virtual void UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle) override;
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override;
	virtual void UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle) override;
	virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override;
	virtual void UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle) override;
	virtual FString InsecureGetLoginToken(const FString& PlayerName) override;
	virtual FString InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override;
	// ~End IVoiceChatUser Interface

	IEOSPlatformHandlePtr GetPlatformHandle() const { return EosPlatformHandle; }
	EOS_HRTC GetRtcInterface() const { return InitSession.EosRtcInterface; }
	EOS_HLobby GetLobbyInterface() const { return InitSession.EosLobbyInterface; }

protected:
	virtual void PostInitialize() {};
	virtual void PreUninitialize() {};

	friend class FEOSVoiceChatUser;
	friend class FEOSVoiceChatFactory;

	// When initialized, contains the state for this initialize session. Reset during Uninitialize.
	enum class EInitializationState
	{
		Uninitialized,
		Uninitializing,
		Initializing,
		Initialized
	};

	struct FInitSession
	{
		FInitSession();

		// `FEOSAudioDevicePool` holds a ref to an object field itself which can't be copied or moved
		UE_NONCOPYABLE(FInitSession)

		// reverts fields to their default state
		void Reset();

		EInitializationState State = EInitializationState::Uninitialized;

		TArray<FOnVoiceChatUninitializeCompleteDelegate> UninitializeCompleteDelegates;

		EOS_HRTC EosRtcInterface = nullptr;
		EOS_HLobby EosLobbyInterface = nullptr;

		EOS_NotificationId OnAudioDevicesChangedNotificationId = EOS_INVALID_NOTIFICATIONID;

		TSharedRef<FEOSAudioDevicePool> EosAudioDevicePool;
	};
	FInitSession InitSession;

	void BindInitCallbacks();
	void UnbindInitCallbacks();
	static void EOS_CALL OnAudioDevicesChangedStatic(const EOS_RTCAudio_AudioDevicesChangedCallbackInfo* CallbackInfo);
	void OnAudioDevicesChanged();

	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);
	void LoadConfig();

	// The current state of the connection.
	enum class EConnectionState
	{
		Disconnected,
		Disconnecting,
		Connecting,
		Connected
	};
	EConnectionState ConnectionState = EConnectionState::Disconnected;

	// IVoiceChat Delegates
	FOnVoiceChatConnectedDelegate OnVoiceChatConnectedDelegate;
	FOnVoiceChatDisconnectedDelegate OnVoiceChatDisconnectedDelegate;
	FOnVoiceChatReconnectedDelegate OnVoiceChatReconnectedDelegate;
	// IVoiceChatUser Delegates
	FOnVoiceChatAvailableAudioDevicesChangedDelegate OnVoiceChatAvailableAudioDevicesChangedDelegate;

	TArray<FEOSVoiceChatUserRef> VoiceChatUsers;
	TArray<FEOSVoiceChatUserRef> ReleasedVoiceChatUsers;
	void ScheduleReleaseUser(IVoiceChatUser* User);

	FEOSVoiceChatUser& GetVoiceChatUser();
	FEOSVoiceChatUser& GetVoiceChatUser() const;
	FEOSVoiceChatUser* SingleUserVoiceChatUser = nullptr;

	// ~Begin FSelfRegisteringExec Interface
	bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	// ~End FSelfRegisteringExec Interface
	TArray<IVoiceChatUser*> UsersCreatedByConsoleCommand;

	IEOSSDKManager& SDKManager;
	IEOSPlatformHandlePtr EosPlatformHandle = nullptr;
	virtual IEOSPlatformHandlePtr EOSPlatformCreate(EOS_Platform_Options& PlatformOptions);
	// Override for platform config name, set via SetPlatformConfigName(). If set, Initialize() uses this instead of reading from config file
	TOptional<FString> OverridePlatformConfigName;

#if !UE_BUILD_SHIPPING
	uint8 InitializationPath = 0;
#endif // !UE_BUILD_SHIPPING

	static int64 StaticInstanceIdCount;
	int64 InstanceId = StaticInstanceIdCount++;

	friend const TCHAR* LexToString(FEOSVoiceChat::EConnectionState State);
};

#endif // WITH_EOSVOICECHAT
