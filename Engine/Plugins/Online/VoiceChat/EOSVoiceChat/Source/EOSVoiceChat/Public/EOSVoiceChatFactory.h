// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"

#if WITH_EOSVOICECHAT

#include "Features/IModularFeatures.h"
#include "Misc/CoreMisc.h"
#include "Templates/SharedPointer.h"
#include "IEOSSDKManager.h"

using IVoiceChatPtr = TSharedPtr<class IVoiceChat, ESPMode::ThreadSafe>;
using IVoiceChatWeakPtr = TWeakPtr<class IVoiceChat, ESPMode::ThreadSafe>;

class FEOSVoiceChatFactory : public IModularFeature, public FSelfRegisteringExec
{
public:
	static FEOSVoiceChatFactory* Get()
	{
		if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
		{
			return &IModularFeatures::Get().GetModularFeature<FEOSVoiceChatFactory>(GetModularFeatureName());
		}
		return nullptr;
	}

	static FName GetModularFeatureName()
	{
		static const FName FeatureName = TEXT("EOSVoiceChatFactory");
		return FeatureName;
	}

	virtual ~FEOSVoiceChatFactory() = default;

	/** Create an instance with its own EOS platform. */
	EOSVOICECHAT_API IVoiceChatPtr CreateInstance();
	/** Create an instance sharing an existing EOS platform. Used to enable interaction with e.g. lobby voice channels via an IVoiceChat interface. */
	EOSVOICECHAT_API IVoiceChatPtr CreateInstanceWithPlatform(const IEOSPlatformHandlePtr& PlatformHandle);
	/** Create an instance using a specific named platform config. Used to enable interaction with e.g. lobby voice channels via an IVoiceChat interface. */
	EOSVOICECHAT_API IVoiceChatPtr CreateInstanceWithPlatformConfig(const FString& PlatformConfigName);
	// ~Begin FSelfRegisteringExec
#if UE_ALLOW_EXEC_COMMANDS
	EOSVOICECHAT_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif
	// ~End FSelfRegisteringExec

private:
	TArray<IVoiceChatWeakPtr> Instances;
};

#endif // WITH_EOSVOICECHAT
