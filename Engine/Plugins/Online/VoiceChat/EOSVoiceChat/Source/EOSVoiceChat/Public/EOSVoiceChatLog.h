// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h" // IWYU pragma: keep
#include "EOSShared.h" // IWYU pragma: keep
#include "Logging/LogMacros.h"

#if WITH_EOSVOICECHAT

DECLARE_LOG_CATEGORY_EXTERN(LogEOSVoiceChat, Log, All);

#define EOSVOICECHATUSER_LOG(EOSVoiceChatLogLevel, EOSVoiceChatFormatStr, ...) \
{ \
	UE_LOG(LogEOSVoiceChat, EOSVoiceChatLogLevel, TEXT("[%p] ") EOSVoiceChatFormatStr, (void*)this, ##__VA_ARGS__); \
}

#define EOSVOICECHATUSER_CLOG(Condition, EOSVoiceChatLogLevel, EOSVoiceChatFormatStr, ...) \
{ \
	UE_CLOG(Condition, LogEOSVoiceChat, EOSVoiceChatLogLevel, TEXT("[%p] ") EOSVoiceChatFormatStr, (void*)this, ##__VA_ARGS__); \
}

#endif // WITH_EOSVOICECHAT
