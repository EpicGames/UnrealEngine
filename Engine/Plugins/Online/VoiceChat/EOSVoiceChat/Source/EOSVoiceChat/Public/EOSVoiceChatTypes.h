// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonSerializerMacros.h"

/**
 * Passed in the ChannelCredentials parameter of EOSVoiceChatUser::JoinChannel calls.
 */
struct FEOSVoiceChatChannelCredentials
	: public FJsonSerializable
{
	FString OverrideUserId;
	FString ClientBaseUrl;
	FString ParticipantToken;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("override_userid", OverrideUserId);
		JSON_SERIALIZE("client_base_url", ClientBaseUrl);
		JSON_SERIALIZE("participant_token", ParticipantToken);
	END_JSON_SERIALIZER
};