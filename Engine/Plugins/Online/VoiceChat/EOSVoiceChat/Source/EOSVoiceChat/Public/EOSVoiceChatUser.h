// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSVoiceChat.h"
#include "EOSVoiceChatLog.h"

#if WITH_EOSVOICECHAT

#include "eos_rtc_types.h"
#include "eos_rtc_audio_types.h"
#include "eos_types.h"

class FEOSVoiceChatUser : public TSharedFromThis<FEOSVoiceChatUser, ESPMode::ThreadSafe>, public IVoiceChatUser
{
public:
	EOSVOICECHAT_API FEOSVoiceChatUser(FEOSVoiceChat& InEOSVoiceChat);
	EOSVOICECHAT_API virtual ~FEOSVoiceChatUser();

	FEOSVoiceChatUser(const FEOSVoiceChatUser&) = delete;

	// ~Begin IVoiceChatUser Interface
	EOSVOICECHAT_API virtual void SetSetting(const FString& Name, const FString& Value) override;
	EOSVOICECHAT_API virtual FString GetSetting(const FString& Name) override;
	EOSVOICECHAT_API virtual void SetAudioInputVolume(float Volume) override;
	EOSVOICECHAT_API virtual void SetAudioOutputVolume(float Volume) override;
	EOSVOICECHAT_API virtual float GetAudioInputVolume() const override;
	EOSVOICECHAT_API virtual float GetAudioOutputVolume() const override;
	EOSVOICECHAT_API virtual void SetAudioInputDeviceMuted(bool bIsMuted) override;
	EOSVOICECHAT_API virtual void SetAudioOutputDeviceMuted(bool bIsMuted) override;
	EOSVOICECHAT_API virtual bool GetAudioInputDeviceMuted() const override;
	EOSVOICECHAT_API virtual bool GetAudioOutputDeviceMuted() const override;
	EOSVOICECHAT_API virtual TArray<FVoiceChatDeviceInfo> GetAvailableInputDeviceInfos() const override;
	EOSVOICECHAT_API virtual TArray<FVoiceChatDeviceInfo> GetAvailableOutputDeviceInfos() const override;
	virtual FOnVoiceChatAvailableAudioDevicesChangedDelegate& OnVoiceChatAvailableAudioDevicesChanged() override { return EOSVoiceChat.OnVoiceChatAvailableAudioDevicesChangedDelegate; }
	EOSVOICECHAT_API virtual void SetInputDeviceId(const FString& InputDeviceId) override;
	EOSVOICECHAT_API virtual void SetOutputDeviceId(const FString& OutputDeviceId) override;
	EOSVOICECHAT_API virtual FVoiceChatDeviceInfo GetInputDeviceInfo() const override;
	EOSVOICECHAT_API virtual FVoiceChatDeviceInfo GetOutputDeviceInfo() const override;
	EOSVOICECHAT_API virtual FVoiceChatDeviceInfo GetDefaultInputDeviceInfo() const override;
	EOSVOICECHAT_API virtual FVoiceChatDeviceInfo GetDefaultOutputDeviceInfo() const override;
	EOSVOICECHAT_API virtual void Login(FPlatformUserId PlatformId, const FString& PlayerName, const FString& Credentials, const FOnVoiceChatLoginCompleteDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void Logout(const FOnVoiceChatLogoutCompleteDelegate& Delegate) override;
	EOSVOICECHAT_API virtual bool IsLoggingIn() const override;
	EOSVOICECHAT_API virtual bool IsLoggedIn() const override;
	virtual FOnVoiceChatLoggedInDelegate& OnVoiceChatLoggedIn() override { return OnVoiceChatLoggedInDelegate; }
	virtual FOnVoiceChatLoggedOutDelegate& OnVoiceChatLoggedOut() override { return OnVoiceChatLoggedOutDelegate; }
	EOSVOICECHAT_API virtual FString GetLoggedInPlayerName() const override;
	EOSVOICECHAT_API virtual void BlockPlayers(const TArray<FString>& PlayerNames) override;
	EOSVOICECHAT_API virtual void UnblockPlayers(const TArray<FString>& PlayerNames) override;
	EOSVOICECHAT_API virtual void JoinChannel(const FString& ChannelName, const FString& ChannelCredentials, EVoiceChatChannelType ChannelType, const FOnVoiceChatChannelJoinCompleteDelegate& Delegate, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override;
	EOSVOICECHAT_API virtual void LeaveChannel(const FString& Channel, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate) override;
	virtual FOnVoiceChatChannelJoinedDelegate& OnVoiceChatChannelJoined() override { return OnVoiceChatChannelJoinedDelegate; }
	virtual FOnVoiceChatChannelExitedDelegate& OnVoiceChatChannelExited() override { return OnVoiceChatChannelExitedDelegate; }
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FOnVoiceChatCallStatsUpdatedDelegate& OnVoiceChatCallStatsUpdated() override { return OnVoiceChatCallStatsUpdatedDelegate; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FOnVoiceChatCallStatsUpdatedDelegate2& OnVoiceChatCallStatsUpdated2() override { return OnVoiceChatCallStatsUpdatedDelegate2; }
	virtual TOptional<FVoiceChatCallStats> GetChannelCallStats(const FString& ChannelName) const override { return {}; }
	EOSVOICECHAT_API virtual void Set3DPosition(const FString& ChannelName, const FVector& Position) override;
	EOSVOICECHAT_API virtual TArray<FString> GetChannels() const override;
	EOSVOICECHAT_API virtual TArray<FString> GetPlayersInChannel(const FString& ChannelName) const override;
	EOSVOICECHAT_API virtual EVoiceChatChannelType GetChannelType(const FString& ChannelName) const override;
	virtual FOnVoiceChatPlayerAddedDelegate& OnVoiceChatPlayerAdded() override { return OnVoiceChatPlayerAddedDelegate; }
	virtual FOnVoiceChatPlayerRemovedDelegate& OnVoiceChatPlayerRemoved() override { return OnVoiceChatPlayerRemovedDelegate; }
	EOSVOICECHAT_API virtual bool IsPlayerTalking(const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerTalkingUpdatedDelegate& OnVoiceChatPlayerTalkingUpdated() override { return OnVoiceChatPlayerTalkingUpdatedDelegate; }
	EOSVOICECHAT_API virtual void SetPlayerMuted(const FString& PlayerName, bool bAudioMuted) override;
	EOSVOICECHAT_API virtual bool IsPlayerMuted(const FString& PlayerName) const override;
	EOSVOICECHAT_API virtual void SetChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName, bool bAudioMuted) override;
	EOSVOICECHAT_API virtual bool IsChannelPlayerMuted(const FString& ChannelName, const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerMuteUpdatedDelegate& OnVoiceChatPlayerMuteUpdated() override { return OnVoiceChatPlayerMuteUpdatedDelegate; }
	EOSVOICECHAT_API virtual void SetPlayerVolume(const FString& PlayerName, float Volume) override;
	EOSVOICECHAT_API virtual float GetPlayerVolume(const FString& PlayerName) const override;
	virtual FOnVoiceChatPlayerVolumeUpdatedDelegate& OnVoiceChatPlayerVolumeUpdated() override { return OnVoiceChatPlayerVolumeUpdatedDelegate; }
	EOSVOICECHAT_API virtual void TransmitToAllChannels() override;
	EOSVOICECHAT_API virtual void TransmitToNoChannels() override;
	EOSVOICECHAT_API virtual void TransmitToSpecificChannels(const TSet<FString>& ChannelNames) override;
	EOSVOICECHAT_API virtual EVoiceChatTransmitMode GetTransmitMode() const override;
	EOSVOICECHAT_API virtual TSet<FString> GetTransmitChannels() const override;
	EOSVOICECHAT_API virtual FDelegateHandle StartRecording(const FOnVoiceChatRecordSamplesAvailableDelegate::FDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void StopRecording(FDelegateHandle Handle) override;
	EOSVOICECHAT_API virtual FDelegateHandle RegisterOnVoiceChatAfterCaptureAudioReadDelegate(const FOnVoiceChatAfterCaptureAudioReadDelegate2::FDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void UnregisterOnVoiceChatAfterCaptureAudioReadDelegate(FDelegateHandle Handle) override;
	EOSVOICECHAT_API virtual FDelegateHandle RegisterOnVoiceChatBeforeCaptureAudioSentDelegate(const FOnVoiceChatBeforeCaptureAudioSentDelegate2::FDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void UnregisterOnVoiceChatBeforeCaptureAudioSentDelegate(FDelegateHandle Handle) override;
	EOSVOICECHAT_API virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void UnregisterOnVoiceChatBeforeRecvMixedAudioRenderedDelegate(FDelegateHandle Handle) override;
	EOSVOICECHAT_API virtual FDelegateHandle RegisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(const FOnVoiceChatBeforeRecvAudioRenderedDelegate::FDelegate& Delegate) override;
	EOSVOICECHAT_API virtual void UnregisterOnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate(FDelegateHandle Handle) override;
	EOSVOICECHAT_API virtual FString InsecureGetLoginToken(const FString& PlayerName) override;
	EOSVOICECHAT_API virtual FString InsecureGetJoinToken(const FString& ChannelName, EVoiceChatChannelType ChannelType, TOptional<FVoiceChatChannel3dProperties> Channel3dProperties = TOptional<FVoiceChatChannel3dProperties>()) override;
	// ~End IVoiceChatUser Interface

	EOSVOICECHAT_API bool AddLobbyRoom(const FString& LobbyId);
	EOSVOICECHAT_API bool RemoveLobbyRoom(const FString& LobbyId);

protected:
	friend class FEOSVoiceChat;

	// Global state for a given user.
	struct FGlobalParticipant
	{
		// The player id
		FString PlayerName;
		// Current talking state (we cache this on the global participant in addition to the channel one)
		bool bTalking = false;
		// Desired block state
		bool bBlocked = false;
		// Desired mute state
		bool bAudioMuted = false;
		// Desired Volume
		float Volume = 1.f;
	};

	/**
	 * TODO some of these "current states" e.g. bAudioMuted don't work correctly / the same way they do in VivoxVoiceChat right now,
	 * they don't reflect the current state on the API side as they do in VivoxVoiceChat (i.e. updated by API callbacks),
	 * they are set to whatever the desired state is just before we call the API to update them. Need to move to setting them
	 * when we receive the success callback.
	 */ 
	// Current state of a user in a channel
	struct FChannelParticipant
	{
		// The player id
		FString PlayerName;
		// Current talking state
		bool bTalking = false;
		// Combined audio mute and isListening state
		bool bAudioDisabled = false;
		// Desired channel mute state
		bool bMutedInChannel = false;
		// Current audio status
		TOptional<EOS_ERTCAudioStatus> AudioStatus;
	};

	enum class EChannelJoinState
	{
		NotJoined,
		Leaving,
		Joining,
		Joined
	};

	// Sending options passed to/received from UpdateSending and its completion callback
	struct FSendingState
	{
		// Microphone input
		bool bAudioEnabled = true;
	};

	// Representation of a particular channel
	struct FChannelSession
	{
		FChannelSession() = default;
		FChannelSession(FChannelSession&&) = default;
		~FChannelSession();

		FChannelSession(const FChannelSession&) = delete;

		bool IsLocalUser(const FChannelParticipant& Participant);
		bool IsLobbySession() const;
		
		// The channel name
		FString ChannelName;
		// The channel type
		EVoiceChatChannelType ChannelType = EVoiceChatChannelType::NonPositional;
		// Current join state of the channel
		EChannelJoinState JoinState = EChannelJoinState::NotJoined;
		// Name of the local player in the channel (can differ from LoginSession.PlayerName if an override ID was provided to JoinChannel)
		FString PlayerName;
		// Current participants in the channel, and the current blocked/muted state
		TMap<FString, FChannelParticipant> Participants;
		// Did the user toggle "off" this channel
		bool bIsNotListening = false;

		// Lobby Id, only relevant for lobby rooms
		FString LobbyId;
		// Lobby channel connection state, only relevant for lobby rooms
		bool bLobbyChannelConnected = false;

		// Desired sending state
		FSendingState DesiredSendingState;
		// Active sending state
		FSendingState ActiveSendingState;

		// Set by JoinChannel and fired on success/failure
		FOnVoiceChatChannelJoinCompleteDelegate JoinDelegate;
		// Set by LeaveChannel and fired on success/failure
		TArray<FOnVoiceChatChannelLeaveCompleteDelegate> LeaveDelegates;

		// Handles for channel callbacks
		EOS_NotificationId OnChannelDisconnectedNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnParticipantStatusChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnParticipantAudioUpdatedNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnAudioBeforeSendNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnMixedAudioBeforeRenderNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnUnmixedAudioBeforeRenderNotificationId = EOS_INVALID_NOTIFICATIONID;
		EOS_NotificationId OnAudioInputStateNotificationId = EOS_INVALID_NOTIFICATIONID;

		TUniquePtr<class FCallbackBase> AudioBeforeSendCallback;
	};

	enum class ELoginState
	{
		LoggedOut,
		LoggingOut,
		LoggingIn,
		LoggedIn
	};

	// When logged in, contains the state for the current login session. Reset by Logout
	struct FLoginSession
	{
		FLoginSession() = default;
		FLoginSession& operator=(FLoginSession&&) = default;
		~FLoginSession();

		FLoginSession(const FLoginSession&) = delete;

		// The numeric platform id for the local user
		FPlatformUserId PlatformId = PLATFORMUSERID_NONE;
		// Name of the local player
		FString PlayerName;
		// EpicAccountId of the local player, converted from PlayerName
		EOS_ProductUserId LocalUserProductUserId;
		// Current login state
		ELoginState State = ELoginState::LoggedOut;
		// Set of channels the user is interacting with
		TMap<FString, FChannelSession> ChannelSessions;
		// Contains participants from all channels and the desired blocked/muted state
		TMap<FString, FGlobalParticipant> Participants;

		// Maps LobbyId to RTC ChannelName, only relevant when VoiceChatFlowMode == Lobby
		TMap<FString, FString> LobbyIdToChannelName;

		// State while handling a logout request
		struct FLogoutState
		{
			FVoiceChatResult Result = FVoiceChatResult::CreateSuccess();
			TSet<FString> ChannelNamesExpectingCallback;
			// Delegates to fire when Logout completes
			TArray<FOnVoiceChatLogoutCompleteDelegate> CompletionDelegates;
		};
		TOptional<FLogoutState> LogoutState;

		// Handles for callbacks
		EOS_NotificationId OnLobbyChannelConnectionChangedNotificationId = EOS_INVALID_NOTIFICATIONID;
	};
	FLoginSession LoginSession;

	// Audio Output state
	struct FAudioOutputOptions
	{
		bool bMuted = false;
		float Volume = 1.0f;
		TOptional<FVoiceChatDeviceInfo> SpecificDeviceInfo;
	};
	FAudioOutputOptions AudioOutputOptions;

	// Audio Input state
	struct FAudioInputOptions
	{
		bool bMuted = false;
		bool bPlatformAEC = false;
		float Volume = 1.0f;
		TOptional<FVoiceChatDeviceInfo> SpecificDeviceInfo;
	};
	FAudioInputOptions AudioInputOptions;

	struct FTransmitState
	{
		EVoiceChatTransmitMode Mode = EVoiceChatTransmitMode::All;
		TSet<FString> SpecificChannels;
	};
	FTransmitState TransmitState;

	// IVoiceChatUser Delegates
	FOnVoiceChatLoggedInDelegate OnVoiceChatLoggedInDelegate;
	FOnVoiceChatLoggedOutDelegate OnVoiceChatLoggedOutDelegate;
	FOnVoiceChatChannelJoinedDelegate OnVoiceChatChannelJoinedDelegate;
	FOnVoiceChatChannelExitedDelegate OnVoiceChatChannelExitedDelegate;
	FOnVoiceChatPlayerAddedDelegate OnVoiceChatPlayerAddedDelegate;
	FOnVoiceChatPlayerTalkingUpdatedDelegate OnVoiceChatPlayerTalkingUpdatedDelegate;
	FOnVoiceChatPlayerMuteUpdatedDelegate OnVoiceChatPlayerMuteUpdatedDelegate;
	FOnVoiceChatPlayerVolumeUpdatedDelegate OnVoiceChatPlayerVolumeUpdatedDelegate;
	FOnVoiceChatPlayerRemovedDelegate OnVoiceChatPlayerRemovedDelegate;
	FOnVoiceChatCallStatsUpdatedDelegate OnVoiceChatCallStatsUpdatedDelegate;
	FOnVoiceChatCallStatsUpdatedDelegate2 OnVoiceChatCallStatsUpdatedDelegate2;

	// Begin IVoiceChatUser Recording Delegates
	FCriticalSection AudioRecordLock;
	FOnVoiceChatRecordSamplesAvailableDelegate OnVoiceChatRecordSamplesAvailableDelegate;

	// Note BeforeCaptureAudioSentLock is used for CaptureAudioRead too as they fire together
	FCriticalSection BeforeCaptureAudioSentLock;
	FOnVoiceChatAfterCaptureAudioReadDelegate2 OnVoiceChatAfterCaptureAudioReadDelegate;
	FOnVoiceChatBeforeCaptureAudioSentDelegate2 OnVoiceChatBeforeCaptureAudioSentDelegate;

	FCriticalSection BeforeRecvMixedAudioRenderedLock;
	FOnVoiceChatBeforeRecvAudioRenderedDelegate OnVoiceChatBeforeRecvMixedAudioRenderedDelegate;
	FCriticalSection BeforeRecvUnmixedAudioRenderedLock;
	FOnVoiceChatBeforeRecvAudioRenderedDelegate OnVoiceChatBeforeRecvUnmixedAudioRenderedDelegate;
	// End IVoiceChatUser Recording Delegates

	FEOSVoiceChat& EOSVoiceChat;

	bool bFakeAudioInput = false;
	bool bInDestructor = false;

	// Helper methods
	EOSVOICECHAT_API bool IsInitialized();
	EOSVOICECHAT_API bool IsConnected();
	EOS_HRTC GetRtcInterface() const { return EOSVoiceChat.GetRtcInterface(); }
	EOS_HLobby GetLobbyInterface() const { return EOSVoiceChat.GetLobbyInterface(); }
	EOSVOICECHAT_API FGlobalParticipant& GetGlobalParticipant(const FString& PlayerName);
	EOSVOICECHAT_API const FGlobalParticipant& GetGlobalParticipant(const FString& PlayerName) const;
	EOSVOICECHAT_API FChannelSession& GetChannelSession(const FString& ChannelName);
	EOSVOICECHAT_API const FChannelSession& GetChannelSession(const FString& ChannelName) const;
	EOSVOICECHAT_API void RemoveChannelSession(const FString& ChannelName);
	EOSVOICECHAT_API void ApplyAudioInputOptions();
	EOSVOICECHAT_API void ApplyAudioOutputOptions();
	EOSVOICECHAT_API void ApplyPlayerBlock(const FGlobalParticipant& GlobalParticipant, const FChannelSession& ChannelSession, FChannelParticipant& ChannelParticipant);
	EOSVOICECHAT_API void ApplyReceivingOptions(const FChannelSession& ChannelSession);
	EOSVOICECHAT_API void ApplyPlayerReceivingOptions(const FString& PlayerName);
	EOSVOICECHAT_API void ApplyPlayerReceivingOptions(const FGlobalParticipant& GlobalParticipant, const FChannelSession& ChannelSession, FChannelParticipant& ChannelParticipant);
	EOSVOICECHAT_API void ApplySendingOptions();
	EOSVOICECHAT_API void ApplySendingOptions(FChannelSession& ChannelSession);
	EOSVOICECHAT_API void BindLoginCallbacks();
	EOSVOICECHAT_API void UnbindLoginCallbacks();
	EOSVOICECHAT_API void BindChannelCallbacks(FChannelSession& ChannelSession);
	EOSVOICECHAT_API void UnbindChannelCallbacks(FChannelSession& ChannelSession);
	EOSVOICECHAT_API void LeaveChannelInternal(const FString& ChannelName, const FOnVoiceChatChannelLeaveCompleteDelegate& Delegate);
	EOSVOICECHAT_API void LogoutInternal(const FOnVoiceChatLogoutCompleteDelegate& Delegate);
	EOSVOICECHAT_API void ClearLoginSession();

	DECLARE_DELEGATE_OneParam(FOnVoiceChatUserRtcRegisterUserCompleteDelegate, const EOS_EResult /* Result */);
	EOSVOICECHAT_API void RtcRegisterUser(const FString& UserId, const FOnVoiceChatUserRtcRegisterUserCompleteDelegate& Delegate);
	DECLARE_DELEGATE_OneParam(FOnVoiceChatUserRtcUnregisterUserCompleteDelegate, const EOS_EResult /* Result */);
	EOSVOICECHAT_API void RtcUnregisterUser(const FString& UserId, const FOnVoiceChatUserRtcUnregisterUserCompleteDelegate& Delegate);
	
	EOSVOICECHAT_API void SetHardwareAECEnabled(bool bEnabled);
    
	// EOS operation callbacks
	static EOSVOICECHAT_API void EOS_CALL OnJoinRoomStatic(const EOS_RTC_JoinRoomCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnJoinRoom(const EOS_RTC_JoinRoomCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnLeaveRoomStatic(const EOS_RTC_LeaveRoomCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnLeaveRoom(const EOS_RTC_LeaveRoomCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnBlockParticipantStatic(const EOS_RTC_BlockParticipantCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnBlockParticipant(const EOS_RTC_BlockParticipantCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnUpdateParticipantVolumeStatic(const EOS_RTCAudio_UpdateParticipantVolumeCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnUpdateParticipantVolume(const EOS_RTCAudio_UpdateParticipantVolumeCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnUpdateReceivingAudioStatic(const EOS_RTCAudio_UpdateReceivingCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnUpdateReceivingAudio(const EOS_RTCAudio_UpdateReceivingCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnUpdateSendingAudio(const EOS_RTCAudio_UpdateSendingCallbackInfo* CallbackInfo);

	// EOS notification callbacks
	static EOSVOICECHAT_API void EOS_CALL OnChannelDisconnectedStatic(const EOS_RTC_DisconnectedCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelDisconnected(const EOS_RTC_DisconnectedCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnLobbyChannelConnectionChangedStatic(const EOS_Lobby_RTCRoomConnectionChangedCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnLobbyChannelConnectionChanged(const EOS_Lobby_RTCRoomConnectionChangedCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnChannelParticipantStatusChangedStatic(const EOS_RTC_ParticipantStatusChangedCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelParticipantStatusChanged(const EOS_RTC_ParticipantStatusChangedCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnChannelParticipantAudioUpdatedStatic(const EOS_RTCAudio_ParticipantUpdatedCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelParticipantAudioUpdated(const EOS_RTCAudio_ParticipantUpdatedCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelAudioBeforeSend(const EOS_RTCAudio_AudioBeforeSendCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnChannelMixedAudioBeforeRenderStatic(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelMixedAudioBeforeRender(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnChannelUnmixedAudioBeforeRenderStatic(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API void OnChannelUnmixedAudioBeforeRender(const EOS_RTCAudio_AudioBeforeRenderCallbackInfo* CallbackInfo);
	static EOSVOICECHAT_API void EOS_CALL OnChannelAudioInputStateStatic(const EOS_RTCAudio_AudioInputStateCallbackInfo* CallbackInfo);
	EOSVOICECHAT_API virtual void OnChannelAudioInputState(const EOS_RTCAudio_AudioInputStateCallbackInfo* CallbackInfo);

	EOSVOICECHAT_API bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	friend const TCHAR* LexToString(ELoginState State);
	friend const TCHAR* LexToString(EChannelJoinState State);
};

#endif // WITH_EOSVOICECHAT
